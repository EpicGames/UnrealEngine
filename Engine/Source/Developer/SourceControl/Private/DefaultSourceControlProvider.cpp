// Copyright Epic Games, Inc. All Rights Reserved.

#include "DefaultSourceControlProvider.h"
#include "Logging/MessageLog.h"

#if SOURCE_CONTROL_WITH_SLATE
	#include "Widgets/SNullWidget.h"
#endif

#define LOCTEXT_NAMESPACE "DefaultSourceControlProvider"

void FDefaultSourceControlProvider::Init(bool bForceConnection)
{
	FMessageLog("SourceControl").Info(LOCTEXT("SourceControlDisabled", "Revision control is disabled"));
}

void FDefaultSourceControlProvider::Close()
{

}

FText FDefaultSourceControlProvider::GetStatusText() const
{
	return LOCTEXT("SourceControlDisabled", "Revision control is disabled");
}


TMap<ISourceControlProvider::EStatus, FString> FDefaultSourceControlProvider::GetStatus() const
{
	TMap<EStatus, FString> Result;
	Result.Add(EStatus::Enabled, IsEnabled() ? TEXT("Yes") : TEXT("No") );
	Result.Add(EStatus::Connected, (IsEnabled() && IsAvailable()) ? TEXT("Yes") : TEXT("No") );
	return Result;
}

bool FDefaultSourceControlProvider::IsAvailable() const
{
	return false;
}

bool FDefaultSourceControlProvider::IsEnabled() const
{
	return false;
}

const FName& FDefaultSourceControlProvider::GetName(void) const
{
	static FName ProviderName("None"); 
	return ProviderName; 
}

ECommandResult::Type FDefaultSourceControlProvider::GetState( const TArray<FString>& InFiles, TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> >& OutState, EStateCacheUsage::Type InStateCacheUsage )
{
	return ECommandResult::Failed;
}

ECommandResult::Type FDefaultSourceControlProvider::GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	return ECommandResult::Failed;
}

TArray<FSourceControlStateRef> FDefaultSourceControlProvider::GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const
{
	return TArray<FSourceControlStateRef>();
}

FDelegateHandle FDefaultSourceControlProvider::RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged )
{
	return FDelegateHandle();
}

void FDefaultSourceControlProvider::UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle )
{

}

ECommandResult::Type FDefaultSourceControlProvider::Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate )
{
	return ECommandResult::Failed;
}

bool FDefaultSourceControlProvider::CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const
{
	return false;
}

bool FDefaultSourceControlProvider::CanCancelOperation( const FSourceControlOperationRef& InOperation ) const
{
	return false;
}

void FDefaultSourceControlProvider::CancelOperation( const FSourceControlOperationRef& InOperation )
{
}

bool FDefaultSourceControlProvider::UsesLocalReadOnlyState() const
{
	return true;
}

bool FDefaultSourceControlProvider::UsesChangelists() const
{
	return false;
}

bool FDefaultSourceControlProvider::UsesUncontrolledChangelists() const
{
	return true;
}

bool FDefaultSourceControlProvider::UsesCheckout() const
{
	return false;
}

bool FDefaultSourceControlProvider::UsesFileRevisions() const
{
	return true;
}

bool FDefaultSourceControlProvider::UsesSnapshots() const
{
	return false;
}

bool FDefaultSourceControlProvider::UsesSoftRevertOnDelete() const
{
	return false;
}

bool FDefaultSourceControlProvider::AllowsDiffAgainstDepot() const
{
	return true;
}

TOptional<bool> FDefaultSourceControlProvider::HasChangesToSync() const
{
	return TOptional<bool>();
}

TOptional<bool> FDefaultSourceControlProvider::HasChangesToCheckIn() const
{
	return TOptional<bool>();
}

void FDefaultSourceControlProvider::Tick()
{

}

TArray< TSharedRef<ISourceControlLabel> > FDefaultSourceControlProvider::GetLabels( const FString& InMatchingSpec ) const
{
	return TArray< TSharedRef<ISourceControlLabel> >();
}

TArray<FSourceControlChangelistRef> FDefaultSourceControlProvider::GetChangelists( EStateCacheUsage::Type InStateCacheUsage )
{
	return TArray<FSourceControlChangelistRef>();
}

#if SOURCE_CONTROL_WITH_SLATE
TSharedRef<class SWidget> FDefaultSourceControlProvider::MakeSettingsWidget() const
{
	return SNullWidget::NullWidget;
}
#endif // SOURCE_CONTROL_WITH_SLATE

#undef LOCTEXT_NAMESPACE
