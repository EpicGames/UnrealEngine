// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlOperation.h"
#include "ISourceControlState.h"
#include "ISourceControlProvider.h"

/**
 * Default source control provider implementation - "None"
 */
class FDefaultSourceControlProvider : public ISourceControlProvider
{
public:
	// ISourceControlProvider implementation
	virtual void Init(bool bForceConnection = true) override;
	virtual void Close() override;
	virtual FText GetStatusText() const override;
	virtual TMap<EStatus, FString> GetStatus() const override;
	virtual bool IsAvailable() const override;
	virtual bool IsEnabled() const override;
	virtual const FName& GetName(void) const override;
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) override { return false; }
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRoot) override {}
	virtual int32 GetStateBranchIndex(const FString& BranchName) const override { return INDEX_NONE; }
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const override { return false; }
	virtual ECommandResult::Type GetState( const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage ) override;
	virtual ECommandResult::Type GetState(const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage) override;
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const override;
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged ) override;
	virtual void UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle ) override;
	virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() ) override;
	virtual bool CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual bool CanCancelOperation( const FSourceControlOperationRef& InOperation ) const override;
	virtual void CancelOperation( const FSourceControlOperationRef& InOperation ) override;
	virtual bool UsesLocalReadOnlyState() const override;
	virtual bool UsesChangelists() const override;
	virtual bool UsesUncontrolledChangelists() const override;
	virtual bool UsesCheckout() const override;
	virtual bool UsesFileRevisions() const override;
	virtual bool UsesSnapshots() const override;
	virtual bool UsesSoftRevertOnDelete() const override;
	virtual bool AllowsDiffAgainstDepot() const override;
	virtual TOptional<bool> HasChangesToSync() const override;
	virtual TOptional<bool> HasChangesToCheckIn() const override;
	virtual void Tick() override;
	virtual TArray< TSharedRef<class ISourceControlLabel> > GetLabels( const FString& InMatchingSpec ) const override;
	virtual TArray<FSourceControlChangelistRef> GetChangelists( EStateCacheUsage::Type InStateCacheUsage ) override;
#if SOURCE_CONTROL_WITH_SLATE
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const override;
#endif // SOURCE_CONTROL_WITH_SLATE
};
