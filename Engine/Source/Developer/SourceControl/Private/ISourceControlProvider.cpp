// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISourceControlProvider.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"


#define LOCTEXT_NAMESPACE "ISourceControlProvider"

ISourceControlProvider::FInitResult ISourceControlProvider::Init(EInitFlags Flags)
{
	// TODO: This implementation is for compatibility. Once all providers have their own
	// implementation it should be removed.
	const bool bForceConnection = EnumHasAnyFlags(Flags, EInitFlags::AttemptConnection);
	Init(bForceConnection);

	FInitResult Result;
	Result.bIsAvailable = IsAvailable();

	return Result;
}

ECommandResult::Type ISourceControlProvider::Login(const FString& InPassword, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	TSharedRef<FConnect, ESPMode::ThreadSafe> ConnectOperation = ISourceControlOperation::Create<FConnect>();
	ConnectOperation->SetPassword(InPassword);
	return Execute(ConnectOperation, InConcurrency, InOperationCompleteDelegate);
}

void ISourceControlProvider::RegisterStateBranches(const TArray<FString>& BranchNames, const TArray<FString>& ContentRoots)
{
	if (ContentRoots.Num() > 0)
	{
		ensureMsgf(ContentRoots.Num() >= 1, TEXT("This provider does not support multiple content roots, only using the first provided root (%s)"), *ContentRoots[0]);
		RegisterStateBranches(BranchNames, ContentRoots[0]);
	}
}

ECommandResult::Type ISourceControlProvider::GetState(const TArray<UPackage*>& InPackages, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage)
{
	TArray<FString> Files = SourceControlHelpers::PackageFilenames(InPackages);
	return GetState(Files, OutState, InStateCacheUsage);
}

FSourceControlStatePtr ISourceControlProvider::GetState(const UPackage* InPackage, EStateCacheUsage::Type InStateCacheUsage)
{
	return GetState(SourceControlHelpers::PackageFilename(InPackage), InStateCacheUsage);
}

FSourceControlStatePtr ISourceControlProvider::GetState(const FString& InFile, EStateCacheUsage::Type InStateCacheUsage)
{
	TArray< FSourceControlStateRef > States;
	if (GetState( { InFile }, States, InStateCacheUsage) == ECommandResult::Succeeded)
	{
		if (!States.IsEmpty())
		{
			FSourceControlStateRef State = States[0];
			return State;
		}
	}
	return nullptr;
}

FSourceControlChangelistStatePtr ISourceControlProvider::GetState(const FSourceControlChangelistRef& InChangelist, EStateCacheUsage::Type InStateCacheUsage)
{
	TArray< FSourceControlChangelistStateRef > States;
	if (GetState( { InChangelist }, States, InStateCacheUsage) == ECommandResult::Succeeded)
	{
		if (!States.IsEmpty())
		{
			FSourceControlChangelistStatePtr State = States[0];
			return State;
		}
	}
	return nullptr;
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(InOperation, nullptr, InFiles, InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(InOperation, nullptr, TArray<FString>(), InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, const UPackage* InPackage, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(InOperation, SourceControlHelpers::PackageFilename(InPackage), InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, const FString& InFile, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	TArray<FString> FileArray;
	if (!InFile.IsEmpty())
	{
		FileArray.Add(InFile);
	}
	return Execute(InOperation, FileArray, InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, const TArray<UPackage*>& InPackages, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	TArray<FString> FileArray = SourceControlHelpers::PackageFilenames(InPackages);
	return Execute(InOperation, nullptr, FileArray, InConcurrency, InOperationCompleteDelegate);
}

ECommandResult::Type ISourceControlProvider::Execute(const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const EConcurrency::Type InConcurrency, const FSourceControlOperationComplete& InOperationCompleteDelegate)
{
	return Execute(InOperation, InChangelist, TArray<FString>(), InConcurrency, InOperationCompleteDelegate);
}

TSharedPtr<class ISourceControlLabel> ISourceControlProvider::GetLabel(const FString& InLabelName) const
{
	TArray< TSharedRef<class ISourceControlLabel> > Labels = GetLabels(InLabelName);
	if (Labels.Num() > 0)
	{
		return Labels[0];
	}

	return nullptr;
}

bool ISourceControlProvider::TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const FString& InFile)
{
	TArray<FString> FileArray;
	FileArray.Add(InFile);

	return TryToDownloadFileFromBackgroundThread(InOperation, FileArray);
}

bool ISourceControlProvider::TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const TArray<FString>& InFiles)
{
	// TryToDownloadFileFromBackgroundThread is unsupported by this source control provider
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ProviderName"), FText::FromName(GetName()));
	FText Message = FText::Format(LOCTEXT("UnsupportedOperationTryToDownloadFileFromBackgroundThread", "TryToDownloadFileFromBackgroundThread is not supported by revision control provider '{ProviderName}'"), Arguments);

	InOperation->AddErrorMessge(Message);

	return false;
}

ECommandResult::Type ISourceControlProvider::SwitchWorkspace(FStringView NewWorkspaceName, FSourceControlResultInfo& OutResultInfo, FString* OutOldWorkspaceName)
{
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ProviderName"), FText::FromName(GetName()));
	FText Message = FText::Format(LOCTEXT("UnsupportedOperationSwitchWorkspace", "SwitchWorkspace is not supported by revision control provider '{ProviderName}'"), Arguments);

	OutResultInfo.ErrorMessages.Add(Message);

	return ECommandResult::Failed;
}

#undef LOCTEXT_NAMESPACE
