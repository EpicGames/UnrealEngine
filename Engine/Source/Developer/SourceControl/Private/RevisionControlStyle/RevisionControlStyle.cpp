// Copyright Epic Games, Inc. All Rights Reserved.

#include "RevisionControlStyle/RevisionControlStyle.h"

#if SOURCE_CONTROL_WITH_SLATE

#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"
#include "SlateGlobals.h"

#include "Framework/Application/SlateApplication.h"

TSharedPtr< class ISlateStyle > FRevisionControlStyleManager::DefaultRevisionControlStyleInstance = nullptr;
FName FRevisionControlStyleManager::CurrentRevisionControlStyleName;

FName FDefaultRevisionControlStyle::StyleName("DefaultRevisionControlStyle");

FRevisionControlStyleManager::~FRevisionControlStyleManager()
{
	if (DefaultRevisionControlStyleInstance.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*DefaultRevisionControlStyleInstance);
	}
}

// FRevisionControlStyleManager

void FRevisionControlStyleManager::ReloadTextures()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
}

const ISlateStyle& FRevisionControlStyleManager::Get()
{
	// Find and get the currently active revision control style
	if (const ISlateStyle* RevisionControlStyle = FSlateStyleRegistry::FindSlateStyle(CurrentRevisionControlStyleName))
	{
		return *RevisionControlStyle;
	}

	if (!DefaultRevisionControlStyleInstance.IsValid())
	{
		DefaultRevisionControlStyleInstance = MakeShared<FDefaultRevisionControlStyle>();
		FSlateStyleRegistry::RegisterSlateStyle(*DefaultRevisionControlStyleInstance);
	}

	return *DefaultRevisionControlStyleInstance;
}

FName FRevisionControlStyleManager::GetStyleSetName()
{
	return Get().GetStyleSetName();
}

void FRevisionControlStyleManager::SetActiveRevisionControlStyle(FName InNewActiveRevisionControlStyleName)
{
	// The style needs to be registered with the Slate Style Registry
	if (!FSlateStyleRegistry::FindSlateStyle(InNewActiveRevisionControlStyleName))
	{
		UE_LOGF(LogSlate, Error, "Could not set the active revision control style, make sure the style you are setting exists and is registered with the FSlateStyleRegistry");
	}
	
	CurrentRevisionControlStyleName = InNewActiveRevisionControlStyleName;
}

void FRevisionControlStyleManager::ResetToDefaultRevisionControlStyle()
{
	SetActiveRevisionControlStyle(DefaultRevisionControlStyleInstance ? DefaultRevisionControlStyleInstance->GetStyleSetName() : NAME_None);
}

// FDefaultRevisionControlStyle

FDefaultRevisionControlStyle::FDefaultRevisionControlStyle() : FSlateStyleSet(StyleName)
{
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Any new revision control icons should be added here instead of in StarshipStyle. Any icons there that don't exist here should be moved here and used by calling FRevisionControlStyleManager::Get()
	
	// We use a custom teal color for the branched icons
	BranchedColor = FLinearColor::FromSRGBColor(FColor::FromHex("#00E4A0"));

	StatusCheckedOutColor = FLinearColor::FromSRGBColor(FColor::FromHex("#1FE44B"));
	StatusCheckedOutByOtherUserColor = FLinearColor::FromSRGBColor(FColor::FromHex("#EF3535"));
	StatusNotAtHeadRevisionColor = FLinearColor::FromSRGBColor(FColor::FromHex("#E1FF3D"));

	SnapshotHistoryAdded = FStyleColors::AccentBlue;
	SnapshotHistoryModified = FLinearColor::FromSRGBColor(FColor::FromHex("#F0AD4E"));
	SnapshotHistoryRemoved = FLinearColor::FromSRGBColor(FColor::FromHex("#CD3642"));

	// Status icons
	Set("RevisionControl.Icon", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControl", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Icon.ConnectedBadge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControlBadgeConnected", CoreStyleConstants::Icon16x16, FStyleColors::Success));
	Set("RevisionControl.Icon.WarningBadge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControlBadgeWarning", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	
	// Icons for revision control actions

	Set("RevisionControl.Actions.Sync", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Sync", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.CheckOut", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.SyncAndCheckOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_SyncAndCheckOut", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.MakeWritable", new CORE_IMAGE_BRUSH_SVG("Starship/Common/edit", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Add", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_MarkedForAdd", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Submit", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckIn", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Connect", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControl", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.History", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Recent", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Diff", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Diff", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Revert", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/icon_SCC_Revert", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Merge", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Merge", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Refresh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.ChangeSettings", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/icon_SCC_Change_Source_Control_Settings", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Actions.Rewind", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Rewind", CoreStyleConstants::Icon16x16));

	// Icons representing the various revision control states

	Set("RevisionControl.Locked", new CORE_IMAGE_BRUSH_SVG("Starship/Common/lock", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Unlocked", new CORE_IMAGE_BRUSH_SVG("Starship/Common/lock-unlocked", CoreStyleConstants::Icon16x16));

	Set("RevisionControl.Shelved", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Shelved", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", CoreStyleConstants::Icon16x16, StatusCheckedOutColor));
	
	Set("RevisionControl.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_MarkedForAdd", CoreStyleConstants::Icon16x16, FStyleColors::AccentBlue));
	Set("RevisionControl.MarkedForDelete", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", CoreStyleConstants::Icon16x16, FStyleColors::Error));

	Set("RevisionControl.CheckedOutByOtherUser", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckedOther", CoreStyleConstants::Icon16x16, StatusCheckedOutByOtherUserColor));
	Set("RevisionControl.CheckedOutByOtherUserBadge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckedOtherBadge", CoreStyleConstants::Icon16x16, FStyleColors::AccentBlue));

	Set("RevisionControl.CheckedOutByOtherUserOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckedBranch", CoreStyleConstants::Icon16x16, BranchedColor));
	Set("RevisionControl.CheckedOutByOtherUserOtherBranchBadge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckedOtherBadge", CoreStyleConstants::Icon16x16, FStyleColors::Warning));

	Set("RevisionControl.ModifiedOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", CoreStyleConstants::Icon16x16, BranchedColor));
	Set("RevisionControl.ModifiedBadge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_BranchModifiedBadge", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	Set("RevisionControl.ModifiedLocally", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_ModifiedLocally", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));

	Set("RevisionControl.NotAtHeadRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_NewerVersion", CoreStyleConstants::Icon16x16, StatusNotAtHeadRevisionColor));
	Set("RevisionControl.NotInDepot", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_NotInDepot", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	
	Set("RevisionControl.Branched", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", CoreStyleConstants::Icon16x16, BranchedColor));

	Set("RevisionControl.Conflicted", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Conflicted", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	Set("RevisionControl.Merged", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Merged", CoreStyleConstants::Icon16x16, FStyleColors::Foreground));
	
	// Misc Icons
	Set("RevisionControl.ChangelistsTab", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.ConflictResolutionTab", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Conflicted", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.SnapshotHistoryTab", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Rewind", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.BranchExplorerTab", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_BranchExplorer", CoreStyleConstants::Icon16x16));
	
	Set("RevisionControl.StatusBar.AtLatestRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_StatusRemoteUpToDate", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.StatusBar.NotAtLatestRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_StatusRemoteDownload", CoreStyleConstants::Icon16x16, StatusNotAtHeadRevisionColor));
	Set("RevisionControl.StatusBar.NoLocalChanges", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_StatusLocalUpToDate", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.StatusBar.HasLocalChanges", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_StatusLocalUpload", CoreStyleConstants::Icon16x16, FStyleColors::AccentBlue));
	Set("RevisionControl.StatusBar.Conflict", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Conflicted", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	Set("RevisionControl.StatusBar.ConflictUpcoming", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", CoreStyleConstants::Icon16x16, FStyleColors::Warning));
	Set("RevisionControl.StatusBar.Promote", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Promote", CoreStyleConstants::Icon16x16));

	Set("RevisionControl.ShowMenu.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.ShowMenu.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_MarkedForAdd", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.ShowMenu.CheckedOutByOtherUser", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckedOther", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.ShowMenu.NotAtHeadRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_NewerVersion", CoreStyleConstants::Icon16x16));
	
	Set("RevisionControl.Promote.Large", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Promote_Large", CoreStyleConstants::Icon32x32));

	Set("RevisionControl.ConflictResolution.OpenExternal", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_ConflictResolution_OpenExternal", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.ConflictResolution.Clear", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_ConflictResolution_Clear", CoreStyleConstants::Icon16x16));

	Set("RevisionControl.SnapshotHistory.Added", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Added", CoreStyleConstants::Icon14x14, SnapshotHistoryAdded));
	Set("RevisionControl.SnapshotHistory.Modified", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Modified", CoreStyleConstants::Icon14x14, SnapshotHistoryModified));
	Set("RevisionControl.SnapshotHistory.Removed", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Removed", CoreStyleConstants::Icon14x14, SnapshotHistoryRemoved));

	Set("RevisionControl.SnapshotHistory.ToolTip.Added", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Added", CoreStyleConstants::Icon14x14));
	Set("RevisionControl.SnapshotHistory.ToolTip.Modified", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Modified", CoreStyleConstants::Icon14x14));
	Set("RevisionControl.SnapshotHistory.ToolTip.Removed", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Removed", CoreStyleConstants::Icon14x14));

	Set("RevisionControl.BranchExplorer.BranchList.Branch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Branch", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.BranchExplorer.BranchList.PersonalBranch", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Developer", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.BranchExplorer.Merge", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Merged", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.BranchExplorer.NewBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_NewBranch", CoreStyleConstants::Icon16x16));

	Set("RevisionControl.Cross", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close", CoreStyleConstants::Icon16x16));
	Set("RevisionControl.Inspect", new CORE_IMAGE_BRUSH_SVG("Starship/Common/visible", CoreStyleConstants::Icon16x16));

	Set("RevisionControl.Filter.ResetToDefault", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ResetToDefault", CoreStyleConstants::Icon16x16));

	// Revision Control States
	Set("RevisionControl.VerticalLine", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_VerticalLine", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.VerticalLineStart", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_VerticalLineStart", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.VerticalLineEnd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_VerticalLineEnd", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.VerticalLineDashed", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_VerticalLineDashed", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.CheckCircleLine_Check", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckCircleLine_Check", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.CheckCircleLine_Circle", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckCircleLine_Circle", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.LineCircle", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_LineCircle", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.CheckInAvailable", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckInAvailable", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.Rewound", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_Rewound", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.CheckInAvailableRewound", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_CheckInAvailableRewound", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.ConflictedState", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_ConflictedState", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.BranchPoint_Branch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_BranchCircle_Branch", CoreStyleConstants::Icon26x26));
	Set("RevisionControl.BranchPoint_Circle", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_BranchCircle_Circle", CoreStyleConstants::Icon26x26));

	Set("RevisionControl.File", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_File", CoreStyleConstants::Icon6x8));
	Set("RevisionControl.DiskSize", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/RC_DiskSize", CoreStyleConstants::Icon8x8));
}

FDefaultRevisionControlStyle::~FDefaultRevisionControlStyle()
{
}
	
const FName& FDefaultRevisionControlStyle::GetStyleSetName() const
{
	return StyleName;
}

#endif //SOURCE_CONTROL_WITH_SLATE
