// Copyright Epic Games, Inc. All Rights Reserved.

#if SOURCE_CONTROL_WITH_SLATE

#include "SSourceControlControls.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "Misc/ConfigCacheIni.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "SourceControlCVars.h"
#include "SourceControlProviders.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SSkeinSourceControlWidgets"

FNumConflicts SSourceControlControls::NumConflictsRemaining;
FNumConflicts SSourceControlControls::NumConflictsUpcoming;

FIsEnabled SSourceControlControls::IsSyncLatestEnabled;
FIsEnabled SSourceControlControls::IsCheckInChangesEnabled;
FIsEnabled SSourceControlControls::IsRestoreAsLatestEnabled;

FIsVisible SSourceControlControls::IsSyncLatestVisible;
FIsVisible SSourceControlControls::IsCheckInChangesVisible;
FIsVisible SSourceControlControls::IsRestoreAsLatestVisible;

FOnClicked SSourceControlControls::OnSyncLatestClicked;
FOnClicked SSourceControlControls::OnCheckInChangesClicked;
FOnClicked SSourceControlControls::OnRestoreAsLatestClicked;

static bool DisplaySyncStatus()
{
	const bool bDisplaySourceControlSyncStatus = SourceControlCVars::CVarSourceControlDisplaySyncStatus.GetValueOnAnyThread();

	if (bDisplaySourceControlSyncStatus)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if (SourceControlModule.IsEnabled() &&
			SourceControlModule.GetProvider().IsAvailable() &&
			SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName())
		{
			return true;
		}
	}

	return false;
}

static bool DisplayCheckInStatus()
{
	const bool bDisplaySourceControlCheckInStatus = SourceControlCVars::CVarSourceControlDisplayCheckInStatus.GetValueOnAnyThread();

	if (bDisplaySourceControlCheckInStatus)
	{
		ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
		if (SourceControlModule.IsEnabled() &&
			SourceControlModule.GetProvider().IsAvailable() &&
			SourceControlModule.GetProvider().GetName() == SourceControlProviders::GetUrcProviderName())
		{
			return true;
		}
	}

	return false;
}

static bool DisplayRestoreAsLatestStatus()
{
	// The 'Restore as Latest' button is a replacement for the 'Check in Changes' button.
	return DisplayCheckInStatus();
}

/**
 * Construct this widget
 *
 * @param	InArgs	The declaration data for this widget
 */
void SSourceControlControls::Construct(const FArguments& InArgs)
{
	IsMiddleSeparatorEnabled = InArgs._IsEnabledMiddleSeparator;
	IsRightSeparatorEnabled = InArgs._IsEnabledRightSeparator;

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot() // Check In Changes Button
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText_Static(&SSourceControlControls::GetSourceControlCheckInStatusToolTipText)
			.Visibility_Static(&SSourceControlControls::GetSourceControlCheckInStatusVisibility)
			.IsEnabled_Static(&SSourceControlControls::IsSourceControlCheckInEnabled)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image_Static(&SSourceControlControls::GetSourceControlCheckInStatusIcon)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
					.Text_Static(&SSourceControlControls::GetSourceControlCheckInStatusText)
				]
			]
			.OnClicked_Static(&SSourceControlControls::OnSourceControlCheckInChangesClicked)
		]
		+ SHorizontalBox::Slot() // Restore as Latest button
		.Padding(FMargin(0.0f, 0.0f, 4.0f, 0.0f))
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText_Static(&SSourceControlControls::GetSourceControlRestoreAsLatestToolTipText)
			.Visibility_Static(&SSourceControlControls::GetSourceControlRestoreAsLatestVisibility)
			.IsEnabled_Static(&SSourceControlControls::IsSourceControlRestoreAsLatestEnabled)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image_Static(&SSourceControlControls::GetSourceControlRestoreAsLatestStatusIcon)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
					.Text_Static(&SSourceControlControls::GetSourceControlRestoreAsLatestText)
				]
			]
			.OnClicked_Static(&SSourceControlControls::OnSourceControlRestoreAsLatestClicked)
		]
		+SHorizontalBox::Slot() // Check In Kebab Combo button
		.AutoWidth()
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(7.f, 0.f))
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("StatusBar.StatusBarEllipsisComboButton"))
			.MenuPlacement(MenuPlacement_AboveAnchor)
			.Visibility_Static(&SSourceControlControls::GetSourceControlCheckInStatusVisibility)
			.OnGetMenuContent(InArgs._OnGenerateKebabMenu)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Visibility(this, &SSourceControlControls::GetSourceControlMiddleSeparatorVisibility)
			.Thickness(1.0)
			.Orientation(EOrientation::Orient_Vertical)
		]
		+ SHorizontalBox::Slot() // Sync Latest Button
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText_Static(&SSourceControlControls::GetSourceControlSyncStatusToolTipText)
			.Visibility_Static(&SSourceControlControls::GetSourceControlSyncStatusVisibility)
			.IsEnabled_Static(&SSourceControlControls::IsSourceControlSyncEnabled)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(SImage)
					.Image_Static(&SSourceControlControls::GetSourceControlSyncStatusIcon)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(5, 0, 0, 0))
				[
					SNew(STextBlock)
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
					.Text_Static(&SSourceControlControls::GetSourceControlSyncStatusText)
				]
			]
			.OnClicked_Static(&SSourceControlControls::OnSourceControlSyncClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSeparator)
			.Visibility(this, &SSourceControlControls::GetSourceControlRightSeparatorVisibility)
			.Thickness(1.0)
			.Orientation(EOrientation::Orient_Vertical)
		]
	];
}

int32 SSourceControlControls::GetNumConflictsRemaining()
{
	return NumConflictsRemaining.IsBound() ? NumConflictsRemaining.Execute() : 0;
}

int32 SSourceControlControls::GetNumConflictsUpcoming()
{
	return NumConflictsUpcoming.IsBound() ? NumConflictsUpcoming.Execute() : 0;
}

/** Sync Status */

bool SSourceControlControls::HasSourceControlChangesToSync()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	return SourceControlModule.IsEnabled() &&
		SourceControlModule.GetProvider().IsAvailable() &&
		SourceControlModule.GetProvider().HasChangesToSync().IsSet() &&
		SourceControlModule.GetProvider().HasChangesToSync().GetValue();
}

bool SSourceControlControls::IsSourceControlSyncEnabled()
{
	if (!HasSourceControlChangesToSync())
	{
		return false;
	}

	if (IsSyncLatestEnabled.IsBound())
	{
		return IsSyncLatestEnabled.Execute();
	}

	return false;
}

EVisibility SSourceControlControls::GetSourceControlSyncStatusVisibility()
{
	if (!GIsEditor)
	{
		// Always visible in the Slate Viewer
		return EVisibility::Visible;
	}

	bool bVisibleSourceControlSyncStatus = false;
	if (IsSyncLatestVisible.IsBound())
	{
		bVisibleSourceControlSyncStatus = IsSyncLatestVisible.Execute();
	}

	bool bDisplaySourceControlSyncStatus = DisplaySyncStatus();
	if (bVisibleSourceControlSyncStatus && bDisplaySourceControlSyncStatus)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SSourceControlControls::GetSourceControlRightSeparatorVisibility() const
{
	EVisibility StatusVisibility = GetSourceControlSyncStatusVisibility();
	if (StatusVisibility != EVisibility::Visible)
	{
		return StatusVisibility;
	}

	return IsRightSeparatorEnabled.Get(true) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SSourceControlControls::GetSourceControlSyncStatusText()
{
	if (HasSourceControlChangesToSync())
	{
		return LOCTEXT("SyncLatestButtonNotAtHeadText", "Sync Latest");
	}

	return LOCTEXT("SyncLatestButtonAtHeadText", "At Latest");
}

FText SSourceControlControls::GetSourceControlSyncStatusToolTipText()
{
	if (GetNumConflictsRemaining() > 0)
	{
		return LOCTEXT("SyncLatestButtonNotAtHeadTooltipTextConflict", "Some of your local changes are still in conflict. Click here to review these conflicts.");
	}
	if (GetNumConflictsUpcoming() > 0)
	{
		return LOCTEXT("SyncLatestButtonNotAtHeadTooltipTextConflictUpcoming", "Some of your local changes conflict with later revisions on the project. Sync Latest or Check In Changes to review and address these issues.");
	}
	if (HasSourceControlChangesToSync())
	{
		return LOCTEXT("SyncLatestButtonNotAtHeadTooltipText", "Sync to the latest revision for this project.");
	}

	return LOCTEXT("SyncLatestButtonAtHeadTooltipText", "Currently at the latest revision for this project.");
}

const FSlateBrush* SSourceControlControls::GetSourceControlSyncStatusIcon()
{
	static const FSlateBrush* ConflictBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.Conflict");
	static const FSlateBrush* ConflictUpcomingBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.ConflictUpcoming");
	static const FSlateBrush* AtHeadBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.AtLatestRevision");
	static const FSlateBrush* NotAtHeadBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.NotAtLatestRevision");

	if (GetNumConflictsRemaining() > 0)
	{
		return ConflictBrush;
	}
	if (GetNumConflictsUpcoming() > 0)
	{
		return ConflictUpcomingBrush;
	}
	if (HasSourceControlChangesToSync())
	{
		return NotAtHeadBrush;
	}

	return AtHeadBrush;
}

FReply SSourceControlControls::OnSourceControlSyncClicked()
{
	if (GetNumConflictsRemaining() > 0)
	{
		if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusConflictResolution")))
		{
			CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
		}
	}
	else if (HasSourceControlChangesToSync())
	{
		if (OnSyncLatestClicked.IsBound())
		{
			OnSyncLatestClicked.Execute();
		}
	}

	return FReply::Handled();
}

/** Check-in Status */

bool SSourceControlControls::HasSourceControlChangesToCheckIn()
{
	ISourceControlModule& SourceControlModule = ISourceControlModule::Get();
	if (SourceControlModule.IsEnabled() && SourceControlModule.GetProvider().IsAvailable())
	{
		TOptional<bool> Result = SourceControlModule.GetProvider().HasChangesToCheckIn();
		if (Result.IsSet())
		{
			return Result.GetValue();
		}
	}
	return false;
}

bool SSourceControlControls::IsSourceControlCheckInEnabled()
{
	if (!HasSourceControlChangesToCheckIn())
	{
		return false;
	}

	if (IsCheckInChangesEnabled.IsBound())
	{
		return IsCheckInChangesEnabled.Execute();
	}

	return false;
}

EVisibility SSourceControlControls::GetSourceControlCheckInStatusVisibility()
{
	if (!GIsEditor)
	{
		// Always visible in the Slate Viewer
		return EVisibility::Visible;
	}

	bool bVisibleSourceControlCheckInStatus = false;
	if (IsCheckInChangesVisible.IsBound())
	{
		bVisibleSourceControlCheckInStatus = IsCheckInChangesVisible.Execute();
	}

	bool bDisplaySourceControlCheckInStatus = DisplayCheckInStatus();
	if (bVisibleSourceControlCheckInStatus && bDisplaySourceControlCheckInStatus)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility SSourceControlControls::GetSourceControlMiddleSeparatorVisibility() const
{
	EVisibility StatusVisibility = GetSourceControlCheckInStatusVisibility();
	if (StatusVisibility != EVisibility::Visible)
	{
		return StatusVisibility;
	}

	return IsMiddleSeparatorEnabled.Get(true) ? EVisibility::Visible : EVisibility::Collapsed;
}

FText SSourceControlControls::GetSourceControlCheckInStatusText()
{
	if (HasSourceControlChangesToCheckIn())
	{
		return LOCTEXT("CheckInButtonChangesText", "Check In Changes");
	}

	return LOCTEXT("CheckInButtonNoChangesText", "No Changes");
}

FText SSourceControlControls::GetSourceControlCheckInStatusToolTipText()
{
	if (GetNumConflictsRemaining() > 0)
	{
		return LOCTEXT("CheckInButtonChangesTooltipTextConflict", "Some of your local changes are still in conflict. Click here to review these conflicts.");
	}
	if (GetNumConflictsUpcoming() > 0)
	{
		return LOCTEXT("CheckInButtonChangesTooltipTextConflictUpcoming", "Some of your local changes conflict with later revisions on the project. Sync Latest or Check In Changes to review and address these issues.");
	}
	if (HasSourceControlChangesToCheckIn())
	{
		return LOCTEXT("CheckInButtonChangesTooltipText", "Check in change(s) to this project.");
	}

	return LOCTEXT("CheckInButtonNoChangesTooltipText", "No changes to check in for this project.");
}

const FSlateBrush* SSourceControlControls::GetSourceControlCheckInStatusIcon()
{
	static const FSlateBrush* ConflictBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.Conflict");
	static const FSlateBrush* ConflictUpcomingBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.ConflictUpcoming");
	static const FSlateBrush* NoLocalChangesBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.NoLocalChanges");
	static const FSlateBrush* HasLocalChangesBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.HasLocalChanges");

	if (GetNumConflictsRemaining() > 0)
	{
		return ConflictBrush;
	}
	if (GetNumConflictsUpcoming() > 0)
	{
		return ConflictUpcomingBrush;
	}
	if (HasSourceControlChangesToCheckIn())
	{
		return HasLocalChangesBrush;
	}

	return NoLocalChangesBrush;
}

FReply SSourceControlControls::OnSourceControlCheckInChangesClicked()
{
	if (GetNumConflictsRemaining() > 0)
	{
		if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusConflictResolution")))
		{
			CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
		}
	}
	else if (HasSourceControlChangesToCheckIn())
	{
		if (OnCheckInChangesClicked.IsBound())
		{
			OnCheckInChangesClicked.Execute();
		}
	}

	return FReply::Handled();
}

/** Restore as Latest */

bool SSourceControlControls::IsSourceControlRestoreAsLatestEnabled()
{
	if (IsRestoreAsLatestEnabled.IsBound())
	{
		return IsRestoreAsLatestEnabled.Execute();
	}

	return false;
}

EVisibility SSourceControlControls::GetSourceControlRestoreAsLatestVisibility()
{
	bool bVisibleSourceControlRestoreAsLatestStatus = false;
	if (IsRestoreAsLatestVisible.IsBound())
	{
		bVisibleSourceControlRestoreAsLatestStatus = IsRestoreAsLatestVisible.Execute();
	}

	bool bDisplaySourceControlRestoreAsLatestStatus = DisplayRestoreAsLatestStatus();
	if (bVisibleSourceControlRestoreAsLatestStatus && bDisplaySourceControlRestoreAsLatestStatus)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FText SSourceControlControls::GetSourceControlRestoreAsLatestText()
{
	return LOCTEXT("RestoreAsLatestButtonText", "Restore as Latest");
}

FText SSourceControlControls::GetSourceControlRestoreAsLatestToolTipText()
{
	if (GetNumConflictsRemaining() > 0)
	{
		return LOCTEXT("RestoreAsLatestTooltipTextConflict", "Some of your local changes are still in conflict. Click here to review these conflicts.");
	}
	
	return LOCTEXT("RestoreAsLatestTooltipText", "Restore this revision to be the latest version of the project for all team members.");
}

const FSlateBrush* SSourceControlControls::GetSourceControlRestoreAsLatestStatusIcon()
{
	static const FSlateBrush* ConflictBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.Conflict");
	static const FSlateBrush* PromoteBrush = FRevisionControlStyleManager::Get().GetBrush("RevisionControl.StatusBar.Promote");

	if (GetNumConflictsRemaining() > 0)
	{
		return ConflictBrush;
	}

	return PromoteBrush;
}

FReply SSourceControlControls::OnSourceControlRestoreAsLatestClicked()
{
	if (GetNumConflictsRemaining() > 0)
	{
		if (IConsoleObject* CObj = IConsoleManager::Get().FindConsoleObject(TEXT("UnrealRevisionControl.FocusConflictResolution")))
		{
			CObj->AsCommand()->Execute(/*Args=*/TArray<FString>(), /*InWorld=*/nullptr, *GLog);
		}
	}
	else
	{
		if (OnRestoreAsLatestClicked.IsBound())
		{
			OnRestoreAsLatestClicked.Execute();
		}
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE

#endif // SOURCE_CONTROL_WITH_SLATE