// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlLogin.h"

#if SOURCE_CONTROL_WITH_SLATE

#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/MessageLog.h"
#include "Misc/ConfigCacheIni.h"
#include "SSourceControlPicker.h"
#include "SourceControlHelpers.h"
#include "SourceControlModule.h"
#include "SourceControlOperations.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#if WITH_UNREAL_DEVELOPER_TOOLS
	#include "MessageLogModule.h"
#endif

#define LOCTEXT_NAMESPACE "SSourceControlLogin"

const float SSourceControlLogin::RefreshFrequency = 0.5f;

void SSourceControlLogin::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow;
	SourceControlLoginClosed = InArgs._OnSourceControlLoginClosed;

	ConnectionState = ELoginConnectionState::Disconnected;

	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();

#if WITH_UNREAL_DEVELOPER_TOOLS
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedRef<class IMessageLogListing> MessageLogListing = MessageLogModule.GetLogListing("SourceControl");
#endif

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredWidth(480.f)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f, 16.0f, 16.0f, 0.0f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.0f)
					[
						SNew(SSourceControlPicker)
						.IsEnabled( this, &SSourceControlLogin::AreControlsEnabled )
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SettingsBorder, SBox)
						.Visibility(this, &SSourceControlLogin::GetSettingsVisibility)
						.IsEnabled(this, &SSourceControlLogin::AreControlsEnabled)
						[
							SourceControlModule.GetProvider().MakeSettingsWidget()
						]
					]
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
						.Visibility(this, &SSourceControlLogin::GetDisabledTextVisibility)
						.WrapTextAt(450.0f)
						.Text(LOCTEXT("SourceControlDisabledText", "Revision control is currently disabled.\n\nTo enable, select a provider from the drop-down box above and enter your credentials."))
					]
				]
	#if WITH_UNREAL_DEVELOPER_TOOLS
				+SVerticalBox::Slot()
				.VAlign(VAlign_Top)
				.AutoHeight()
				.Padding(0.0f)
				[
					SNew(SExpandableArea)
					.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
					.Visibility(this, &SSourceControlLogin::GetSettingsVisibility)
					.IsEnabled(this, &SSourceControlLogin::AreControlsEnabled)
					.InitiallyCollapsed(true)
					.HeaderContent()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("LogTitle", "Revision Control Log"))
					]
					.BodyContent()
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.Padding(0)
						[
							SNew(SBox)
							.HeightOverride(250.0f)
							[
								MessageLogModule.CreateLogListingWidget(MessageLogListing)
							]
						]
					]
				]
	#endif
				+SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.Padding(8.0f, 16.0f, 8.0f, 16.0f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SThrobber)
						.Visibility(this, &SSourceControlLogin::GetThrobberVisibility)
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Right)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(5.0f, 0.0f))
						[
							SNew(SButton)
							.VAlign(VAlign_Center)
							.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
  							.TextStyle( FAppStyle::Get(), "DialogButtonText" )
							.Text(LOCTEXT("AcceptSettings", "Accept Settings"))
							.OnClicked( this, &SSourceControlLogin::OnAcceptSettings )
							.IsEnabled( this, &SSourceControlLogin::IsAcceptSettingsEnabled )
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(FMargin(5.0f, 0.0f))
						[
							SNew(SButton)
							.VAlign(VAlign_Center)
  							.TextStyle( FAppStyle::Get(), "DialogButtonText" )
							.Text(LOCTEXT("RunWithoutSourceControl", "Disable Revision Control"))
							.OnClicked( this, &SSourceControlLogin::OnDisableSourceControl )
							.IsEnabled( this, &SSourceControlLogin::AreControlsEnabled )
						]
					]
				]
			]
		]
	];
}

void SSourceControlLogin::RefreshSettings()
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	SettingsBorder->SetContent(SourceControlModule.GetProvider().MakeSettingsWidget());
}

FReply SSourceControlLogin::OnAcceptSettings()
{
	ConnectionState = ELoginConnectionState::Connecting;

	//Increase the tick frequency during login if needed
	if ( ParentWindowPtr.IsValid() && ( FSlateApplication::Get().GetActiveModalWindow() == ParentWindowPtr.Pin() ) )
	{
		ActiveTimerHandle = RegisterActiveTimer( 0.f, FWidgetActiveTimerDelegate::CreateSP( this, &SSourceControlLogin::TickSourceControlModule ) );
	}

	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	ECommandResult::Type Result = SourceControlModule.GetProvider().Login(FString(), EConcurrency::Asynchronous, FSourceControlOperationComplete::CreateSP(this, &SSourceControlLogin::SourceControlOperationComplete));
	if (Result == ECommandResult::Succeeded)
	{
		SaveSettingsToDisk();
	}
	else if(Result == ECommandResult::Failed)
	{
		DisplayConnectionError();
		ConnectionState = ELoginConnectionState::Disconnected;
	}

	return FReply::Handled();
}

bool SSourceControlLogin::ConfirmDisableSourceControl() const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	if (SourceControlModule.GetSourceControlProviderChanging().IsBound())
	{
		return SourceControlModule.GetSourceControlProviderChanging().Execute();
	}
	return true;
}

FReply SSourceControlLogin::OnDisableSourceControl()
{
	if (ConfirmDisableSourceControl() == false) // The user has decided to abort the operation
	{
		return FReply::Handled();
	}

	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	SourceControlModule.SetProvider("None");
	if(ParentWindowPtr.IsValid())
	{
		ParentWindowPtr.Pin()->RequestDestroyWindow();
	}
	SourceControlLoginClosed.ExecuteIfBound(false);

	SaveSettingsToDisk();

	return FReply::Handled();
}

void SSourceControlLogin::SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	if(InResult == ECommandResult::Succeeded)
	{
		ConnectionState = ELoginConnectionState::Connected;
		FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
		SourceControlModule.SaveSettings();
		DisplayConnectionSuccess();
		SourceControlLoginClosed.ExecuteIfBound(true);
		if(ParentWindowPtr.IsValid())
		{
			ParentWindowPtr.Pin()->RequestDestroyWindow();
		}
	}
	else
	{
		ConnectionState = ELoginConnectionState::Disconnected;
		TSharedRef<FConnect, ESPMode::ThreadSafe> ConnectOperation = StaticCastSharedRef<FConnect>(InOperation);
		DisplayConnectionError(ConnectOperation->GetErrorText());
	}
}

void SSourceControlLogin::DisplayConnectionError(const FText& InErrorText)
{
	FMessageLog SourceControlLog("SourceControl");
	if(InErrorText.IsEmpty())
	{
		SourceControlLog.Error(LOCTEXT("FailedToConnect", "Failed to connect to revision control. Check your settings and connection then try again."));
	}
	else
	{
		SourceControlLog.Error(FText::Format(LOCTEXT("ConnectionErrorFormat", "Connection Error: {0}"), InErrorText));
	}
	
	SourceControlLog.Notify();

	// Suspend the active timer until there's another login attempt
	auto PinnedActiveTimer = ActiveTimerHandle.Pin();
	if ( PinnedActiveTimer.IsValid() )
	{
		UnRegisterActiveTimer( PinnedActiveTimer.ToSharedRef() );
	}
}

void SSourceControlLogin::DisplayConnectionSuccess() const
{
	FNotificationInfo Info( LOCTEXT("ConnectionSuccessful", "Connection to revision control was successful!") );
	Info.bFireAndForget = true;
	Info.bUseSuccessFailIcons = true;
	TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info);

	Notification->SetCompletionState(SNotificationItem::CS_Success);
}

EVisibility SSourceControlLogin::GetThrobberVisibility() const
{
	return ConnectionState == ELoginConnectionState::Connecting ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SSourceControlLogin::AreControlsEnabled() const
{
	return ConnectionState == ELoginConnectionState::Disconnected;
}

bool SSourceControlLogin::IsAcceptSettingsEnabled() const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	return AreControlsEnabled() && SourceControlModule.GetProvider().GetName() != "None";
}

EVisibility SSourceControlLogin::GetSettingsVisibility() const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	return SourceControlModule.GetProvider().GetName() == "None" ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SSourceControlLogin::GetDisabledTextVisibility() const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	return SourceControlModule.GetProvider().GetName() == "None" ? EVisibility::Visible : EVisibility::Collapsed;
}

EActiveTimerReturnType SSourceControlLogin::TickSourceControlModule( double InCurrentTime, float InDeltaTime )
{
	FSourceControlModule::Get().Tick();
	return EActiveTimerReturnType::Continue;
}

void SSourceControlLogin::SaveSettingsToDisk() const
{
	check(::IsInGameThread());
	GConfig->Flush(false, USourceControlHelpers::GetSettingsIni());
}

#undef LOCTEXT_NAMESPACE

#endif // SOURCE_CONTROL_WITH_SLATE
