// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if SOURCE_CONTROL_WITH_SLATE

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SCompoundWidget.h"

class FActiveTimerHandle;
class IDetailsView;
class SBox;
class SWindow;

namespace ELoginConnectionState
{
	enum Type
	{
		Disconnected,
		Connecting,
		Connected,
	};
}

class SSourceControlLogin : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSourceControlLogin) {}
	
	/** A reference to the parent window */
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

	/** Callback to be called when the "Disable Revision Control" button is pressed. */
	SLATE_ARGUMENT(FSourceControlLoginClosed, OnSourceControlLoginClosed)

	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

	/**
	 * Refresh the displayed settings. Usually called when a provider is changed.
	 */
	void RefreshSettings();

private:

	/** Ticks the source control module (only necessary if the login window is modal) */
	EActiveTimerReturnType TickSourceControlModule( double InCurrentTime, float InDeltaTime );

	/** Delegate called when the user clicks the 'Accept Settings' button */
	FReply OnAcceptSettings();

	/** Delegate called when the user clicks the 'Disable Source Control' button */
	FReply OnDisableSourceControl();

	/** Called when a connection attempt fails */
	void DisplayConnectionError(const FText& InErrorText = FText());

	/** Called when a connection attempt succeeds */
	void DisplayConnectionSuccess() const;

	/** Delegate called form the source control system when a login attempt has completed */
	void SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Delegate to determine control enabled state */
	bool AreControlsEnabled() const;

	/** Delegate to determine 'accept settings' button enabled state */
	bool IsAcceptSettingsEnabled() const;

	/** Delegate to determine visibility of the throbber */
	EVisibility GetThrobberVisibility() const;

	/** Delegate to determine visibility of the settings widget */
	EVisibility GetSettingsVisibility() const;

	/** Delegate to determine visibility of the disabled text widget */
	EVisibility GetDisabledTextVisibility() const;

	/** Requests confirmation of disabling source control */
	bool ConfirmDisableSourceControl() const;

	/** Flushes all current revision control settings to disk */
	void SaveSettingsToDisk() const;

private:
	/** The frequency at which to tick to scc module when inside a modal window */
	static const float RefreshFrequency;

	/** The parent window of this widget */
	TWeakPtr<SWindow> ParentWindowPtr;

	/** Holds the details view. */
	TSharedPtr<class IDetailsView> DetailsView;

	/** Current connection state */
	ELoginConnectionState::Type ConnectionState;

	/** Delegate called when the window is closed */
	FSourceControlLoginClosed SourceControlLoginClosed;

	/** The currently displayed settings widget container */
	TSharedPtr<SBox> SettingsBorder;

	/** The handle to the active scc module tick */
	TWeakPtr<FActiveTimerHandle> ActiveTimerHandle;
};

#endif // SOURCE_CONTROL_WITH_SLATE
