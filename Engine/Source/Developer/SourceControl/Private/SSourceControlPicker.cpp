// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSourceControlPicker.h"
#if SOURCE_CONTROL_WITH_SLATE
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#endif
#include "ISourceControlModule.h"
#include "SourceControlModule.h"

#if SOURCE_CONTROL_WITH_SLATE

#include "SSourceControlLogin.h"

#define LOCTEXT_NAMESPACE "SSourceControlPicker"

void SSourceControlPicker::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.FillWidth(1.0f)
		.Padding(FMargin(0.0f, 0.0f, 16.0f, 10.0f))
		[
			SNew( STextBlock )
			.Text( LOCTEXT("ProviderLabel", "Provider") )
		]
		+SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f, 0.0f, 0.0f, 10.0f))
		.FillWidth(2.0f)
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &SSourceControlPicker::OnGetMenuContent)
			.ToolTipText( LOCTEXT("ChooseProvider", "Choose the revision control provider you want to use before you edit login settings.") )
			.ButtonContent()
			[
				SNew( STextBlock )
				.Text(this, &SSourceControlPicker::OnGetButtonText)
			]
		]
	];
}

void SSourceControlPicker::ChangeSourceControlProvider(int32 ProviderIndex) const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();

	if (CurrentProviderIndex != ProviderIndex)
	{
		if (ConfirmProviderChanging() == false) // The user has decided to abort the operation
		{
			return;
		}
	}

	SourceControlModule.SetCurrentSourceControlProvider(ProviderIndex);

	if(SourceControlModule.GetLoginWidget().IsValid())
	{
		SourceControlModule.GetLoginWidget()->RefreshSettings();
	}
}

bool SSourceControlPicker::ConfirmProviderChanging() const
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();
	if (SourceControlModule.GetSourceControlProviderChanging().IsBound())
	{
		return SourceControlModule.GetSourceControlProviderChanging().Execute();
	}
	return true;
}

TSharedRef<SWidget> SSourceControlPicker::OnGetMenuContent()
{
	FSourceControlModule& SourceControlModule = FSourceControlModule::Get();

	FMenuBuilder MenuBuilder(true, NULL);

	// Get the provider names first so that we can sort them for the UI
	TArray< TPair<FName, int32> > SortedProviderNames;
	const int NumProviders = SourceControlModule.GetNumSourceControlProviders();
	SortedProviderNames.Reserve(NumProviders);
	for(int ProviderIndex = 0; ProviderIndex < NumProviders; ++ProviderIndex)
	{
		const FName ProviderName = SourceControlModule.GetSourceControlProviderName(ProviderIndex);
		int32 ProviderSortKey = ProviderName == FName("None") ? -1 * ProviderIndex : ProviderIndex;
		SortedProviderNames.Emplace(ProviderName, ProviderSortKey);

		if (ProviderName == FSourceControlModule::Get().GetProvider().GetName())
		{
			CurrentProviderIndex = ProviderIndex;
		}
	}

	// Sort based on the provider index
	SortedProviderNames.Sort([](const TPair<FName, int32>& One, const TPair<FName, int32>& Two)
	{
		return One.Value < Two.Value;
	});

	for(auto SortedProviderNameIter = SortedProviderNames.CreateConstIterator(); SortedProviderNameIter; ++SortedProviderNameIter)
	{
		const FText ProviderText = GetProviderText(SortedProviderNameIter->Key);
		const int32  ProviderIndex = SortedProviderNameIter->Value;

		FFormatNamedArguments Arguments;
		Arguments.Add( TEXT("ProviderName"), ProviderText );
		MenuBuilder.AddMenuEntry(
			ProviderText,
			FText::Format(LOCTEXT("SourceControlProvider_Tooltip", "Use {ProviderName} as revision control provider"), Arguments),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP( this, &SSourceControlPicker::ChangeSourceControlProvider, FMath::Abs(ProviderIndex) ),
				FCanExecuteAction()
				)
			);
	}

	return MenuBuilder.MakeWidget();
}

FText SSourceControlPicker::OnGetButtonText() const
{
	return GetProviderText( ISourceControlModule::Get().GetProvider().GetName() );
}

FText SSourceControlPicker::GetProviderText(const FName& InName) const
{
	if(InName == "None")
	{
		return LOCTEXT("NoProviderDescription", "None  (revision control disabled)");
	}

	// @todo: Remove this block after the Git plugin has been exhaustively tested (also remember to change the Git plugin's "IsBetaVersion" setting to false.)
	if(InName == "Git")
	{
		return LOCTEXT( "GitBetaProviderName", "Git  (beta version)" );
	}

	return FText::FromName(InName);
}
#undef LOCTEXT_NAMESPACE

#endif // SOURCE_CONTROL_WITH_SLATE
