// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#if SOURCE_CONTROL_WITH_SLATE
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Application/SlateWindowHelper.h"
#endif

#if SOURCE_CONTROL_WITH_SLATE


class SSourceControlPicker : public SCompoundWidget
{
public:
	
	SLATE_BEGIN_ARGS(SSourceControlPicker) {}
	
	SLATE_END_ARGS()

public:

	void Construct(const FArguments& InArgs);

private:

	/** Delegate called when changing source control providers */
	void ChangeSourceControlProvider(int32 ProviderIndex) const;

	/** Get the content for the drop-down menu for picking providers */
	TSharedRef<SWidget> OnGetMenuContent();

	/** Get the button text for the drop-down */
	FText OnGetButtonText() const;

	/** Get the text to be displayed given the name of the provider */
	FText GetProviderText(const FName& InName) const;

	/** Requests confirmation for the change of provider */
	bool ConfirmProviderChanging() const;

	/** The index of the currently active provider */
	int CurrentProviderIndex = -1;
};

#endif // SOURCE_CONTROL_WITH_SLATE
