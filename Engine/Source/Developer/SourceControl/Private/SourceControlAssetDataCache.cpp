// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlAssetDataCache.h"

#include "Async/Async.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ISourceControlModule.h"
#include "ISourceControlState.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "SourceControlProviders.h"

#if WITH_EDITOR
#include "GameFramework/Actor.h"
#endif

static TAutoConsoleVariable<int32> CVarSourceControlAssetDataCacheMaxAsyncTask(
	TEXT("SourceControlAssetDataCache.MaxAsyncTask"),
	8,
	TEXT("Maximum number of task running in parallel to fetch AssetData information.")
);

void FSourceControlAssetDataCache::Startup()
{
	ISourceControlModule& SCCModule = ISourceControlModule::Get();

	ProviderChangedDelegateHandle = SCCModule.RegisterProviderChanged(FSourceControlProviderChanged::FDelegate::CreateLambda([](ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
	{
		ISourceControlModule::Get().GetAssetDataCache().OnSourceControlProviderChanged(OldProvider, NewProvider);
	}));

#if WITH_EDITOR
	auto ClearCachedAssetData = [](const UObject* Object)
	{
		// Transient packages should not be saved, therefore we can skip the asset data cache update for them to mitigate the perf impact of this call 
		if (UPackage* Package = Object->GetPackage(); Package && !Package->HasAnyFlags(RF_Transient))
		{
			FString Filename = USourceControlHelpers::PackageFilename(Package);
			ISourceControlModule::Get().GetAssetDataCache().ClearAssetData(Filename);
		}
	};

	ActorLabelChangedDelegateHandle = FCoreDelegates::OnActorLabelChanged.AddLambda(ClearCachedAssetData);

	FCoreDelegates::OnAllModuleLoadingPhasesComplete.AddLambda([this, &ClearCachedAssetData]()
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(FName("AssetRegistry"));
		AssetUpdatedDelegateHandle = AssetRegistryModule.Get().OnAssetUpdated().AddLambda([&ClearCachedAssetData](const FAssetData& AssetData)
		{
			if (UObject* Object = AssetData.FastGetAsset())
			{
				ClearCachedAssetData(Object);
			}
		});
	});
#endif

	bIsSourceControlDialogShown = false;
	bIsUpdateHistoryInFlight = false;
	bIsDownloadFileInFlight = false;
}

void FSourceControlAssetDataCache::Shutdown()
{
#if WITH_EDITOR
	FCoreDelegates::OnActorLabelChanged.Remove(ActorLabelChangedDelegateHandle);

	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>(TEXT("AssetRegistry")))
	{
		AssetRegistryModule->Get().OnAssetUpdated().Remove(AssetUpdatedDelegateHandle);
	}
#endif

	ISourceControlModule& SCCModule = ISourceControlModule::Get();
	SCCModule.UnregisterProviderChanged(ProviderChangedDelegateHandle);

	ClearPendingTasks();
}

bool FSourceControlAssetDataCache::GetAssetDataArray(FSourceControlStateRef InFileState, FAssetDataArrayPtr& OutAssetDataArrayPtr)
{
	if (bIsSourceControlDialogShown)
	{
		return false;
	}

	FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(InFileState->GetFilename());

	if (AssetDataEntry == nullptr)
	{
		AssetDataEntry = AddAssetInformationEntry(InFileState);
	}

	check(AssetDataEntry != nullptr);

	if (AssetDataEntry->bInitialized)
	{
		OutAssetDataArrayPtr = AssetDataEntry->AssetDataArrayPtr;
		return true;
	}

	return false;
}

FSourceControlAssetDataEntry* FSourceControlAssetDataCache::AddAssetInformationEntry(FSourceControlStateRef InFileState)
{
	FString Filename = InFileState->GetFilename();
	FSourceControlAssetDataEntry* AssetDataEntry = &AssetDataCache.Add(Filename);

	check(AssetDataEntry != nullptr);
	check(!AssetDataEntry->AssetDataArrayPtr.IsValid());

	AssetDataEntry->bInitialized = false;
	AssetDataEntry->AssetDataArrayPtr = MakeShared<TArray<FAssetData>>();

	/* For deleted items, the file is not on disk anymore so the only way we can get the asset data is by getting the file from the depot.
	 * For shelved files, if the file still exists locally, it will have been found before, otherwise, the history of the shelved file state will point to the remote version
	 */
	if (!InFileState->IsDeleted())
	{
		AssetDataEntry->bInitialized = USourceControlHelpers::GetAssetData(Filename, *AssetDataEntry->AssetDataArrayPtr);
	}

	// At the moment, getting the asset data from non-external assets yields issues with the package path
	if (IAssetRegistry::Get()->IsPathBeautificationNeeded(Filename) && (InFileState->IsDeleted() || (AssetDataEntry->AssetDataArrayPtr->Num() == 0)))
	{
		check(!AssetDataEntry->bInitialized);

		const bool bPrefersDownloadFile = ISourceControlModule::Get().GetProvider().GetName() == SourceControlProviders::GetUrcProviderName();
		if (bPrefersDownloadFile)
		{
			FilesToDownload.Add(Filename);
		}
		else if (InFileState->GetHistorySize() > 0)
		{
			AssetDataToFetch.Enqueue(InFileState);
		}
		else
		{
			FileHistoryToUpdate.Add(Filename);
		}
	}
	else
	{
		// We either already have AssetData or we can't go further with this asset.
		AssetDataEntry->bInitialized = true;
	}

	return AssetDataEntry;
}

void FSourceControlAssetDataCache::Tick()
{
	DispatchDownloadFile();
	DispatchUpdateHistory();
	DispatchFetchAssetData();

	UpdatePendingAssetData();
}

void FSourceControlAssetDataCache::OnSourceControlDialogShown()
{
	ClearPendingTasks();
	bIsSourceControlDialogShown = true;
}

void FSourceControlAssetDataCache::OnSourceControlDialogClosed()
{
	bIsSourceControlDialogShown = false;
}

void FSourceControlAssetDataCache::DispatchDownloadFile()
{
	if (bIsDownloadFileInFlight || !AssetDataToLoad.IsEmpty())
	{
		return;
	}

	if (FilesToDownload.Num() == 0)
	{
		return;
	}

	const int32 MaxFilesPerRequest = 100;
	const int32 NumItemsAvailable = FilesToDownload.Num();
	const int32 NumItemsToDispatch = FMath::Min(MaxFilesPerRequest, NumItemsAvailable);

	TSet<FString> Filenames;
	TArray<FString> FilesToDispatch;
	Filenames.Reserve(NumItemsToDispatch);
	FilesToDispatch.Reserve(NumItemsToDispatch);

	for (int32 Index = 0; Index < FilesToDownload.Num() && FilesToDispatch.Num() < NumItemsToDispatch; /* */)
	{
		const FString Filename = FPaths::GetCleanFilename(FilesToDownload[Index]);
		if (Filenames.Contains(Filename))
		{
			// In case of /path/to/1/file.uasset and /path/to/2/file.uasset, the filenames will collide in the target directory.
			// So only pass unique filenames to the FDownloadFile operation.
			++Index;
		}
		else
		{
			Filenames.Add(Filename);
			FilesToDispatch.Add(FilesToDownload[Index]);
			FilesToDownload.RemoveAtSwap(Index);
		}
	}

	const FString TargetDirectory = FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("SCCAssetData"), FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower));
	FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*TargetDirectory);

	bIsDownloadFileInFlight = true;

	TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadOperation = ISourceControlOperation::Create<FDownloadFile>(TargetDirectory);
	DownloadOperation->SetEnableErrorLogging(false);

	ISourceControlModule::Get().GetProvider().Execute(DownloadOperation, FilesToDispatch, EConcurrency::Asynchronous,
		FSourceControlOperationComplete::CreateLambda(
			[FilesToDispatch](const FSourceControlOperationRef& Operation, ECommandResult::Type InResult)
			{
				ISourceControlModule::Get().GetAssetDataCache().OnDownloadFileComplete(FilesToDispatch, Operation, InResult);
			}
		)
	);
}

void FSourceControlAssetDataCache::OnDownloadFileComplete(const TArray<FString>& InDispatchedFiles, const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	bIsDownloadFileInFlight = false;

	TSharedRef<FDownloadFile, ESPMode::ThreadSafe> DownloadOperation = StaticCastSharedRef<FDownloadFile>(InOperation);

	const FString DiffDirectory = FPaths::DiffDir();
	const FString TargetDirectory = DownloadOperation->GetTargetDirectory();

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	for (const FString& Filename : InDispatchedFiles)
	{
		FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(Filename);
		if (AssetDataEntry == nullptr || AssetDataEntry->bInitialized)
		{
			continue;
		}

		const FString TempFileName = FPaths::Combine(TargetDirectory, FPaths::GetCleanFilename(Filename));
		if (!PlatformFile.FileExists(*TempFileName) || PlatformFile.FileSize(*TempFileName) == 0)
		{
			// We can't go further with this asset.
			AssetDataEntry->bInitialized = true;
			continue;
		}

		// Move the file to the DiffDir from where its asset data can be loaded - this doesn't work from just any dir.
		const FString DestFileName = FPaths::Combine(DiffDirectory, FString::Printf(TEXT("temp-adc-%s"), *FPaths::GetCleanFilename(Filename)));
		PlatformFile.DeleteFile(*DestFileName);
		PlatformFile.MoveFile(*DestFileName, *TempFileName);

		FAssetDataToLoad ToLoad = { DestFileName, Filename };
		AssetDataToLoad.Enqueue(MoveTemp(ToLoad));
	}

	FPlatformFileManager::Get().GetPlatformFile().DeleteDirectoryRecursively(*TargetDirectory);
}

void FSourceControlAssetDataCache::DispatchUpdateHistory()
{
	if (bIsUpdateHistoryInFlight)
	{
		return;
	}

	if (FileHistoryToUpdate.Num() == 0)
	{
		return;
	}

	const int32 MaxStatusDispatchesPerRequest = 100; // Use a conservative number, SetUpdateHistory(true) could be expensive.
	const int32 NumItemsAvailable = FileHistoryToUpdate.Num();
	const int32 NumItemsToDispatch = FMath::Min(MaxStatusDispatchesPerRequest, NumItemsAvailable);

	TArray<FString> FilesToDispatch;
	FilesToDispatch.Reserve(NumItemsToDispatch);
	for (int32 Index = 0; Index < NumItemsToDispatch; ++Index)
	{
		FilesToDispatch.Add(FileHistoryToUpdate[Index]);
	}
	FileHistoryToUpdate.RemoveAt(0, NumItemsToDispatch);

	bIsUpdateHistoryInFlight = true;

	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	UpdateStatusOperation->SetQuiet(true);

	ISourceControlModule::Get().GetProvider().Execute(UpdateStatusOperation, FilesToDispatch, EConcurrency::Asynchronous,
		FSourceControlOperationComplete::CreateLambda(
			[FilesToDispatch](const FSourceControlOperationRef& Operation, ECommandResult::Type InResult)
			{
				ISourceControlModule::Get().GetAssetDataCache().OnUpdateHistoryComplete(FilesToDispatch, Operation, InResult);
			}
		)
	);
}

void FSourceControlAssetDataCache::OnUpdateHistoryComplete(const TArray<FString>& InUpdatedFiles, const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
{
	bIsUpdateHistoryInFlight = false;

	if (InResult == ECommandResult::Failed)
	{
		UE_LOGF(LogSourceControl, Warning, "AssetDataCache: UpdateHistory Failed.");
		return;
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	for (const FString& Filename : InUpdatedFiles)
	{
		// The AssetDataCache container entry might have been removed during FUpdateStatus.
		if (FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(Filename))
		{
			FSourceControlStatePtr FileState = SourceControlProvider.GetState(Filename, EStateCacheUsage::Use);
			check(FileState.IsValid());

			if (FileState->GetHistorySize() > 0)
			{
				AssetDataToFetch.Enqueue(FileState);
			}
			else
			{
				// History was not updated we probably cannot go further with this asset.
				AssetDataEntry->bInitialized = true;
			}
		}
	}
}

void FSourceControlAssetDataCache::ClearAssetData(const FString& InFilename)
{
	if (FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(InFilename))
	{
		if (AssetDataEntry->bInitialized)
		{
			AssetDataCache.Remove(InFilename);
		}
	}
}

void FSourceControlAssetDataCache::DispatchFetchAssetData()
{
	FSourceControlStatePtr FileState;
	const uint32 MaxAsyncTask = static_cast<uint32>(CVarSourceControlAssetDataCacheMaxAsyncTask.GetValueOnGameThread());

	check(MaxAsyncTask > 0);

	while ((CurrentAsyncTask < MaxAsyncTask) && AssetDataToFetch.Dequeue(FileState))
	{
		check(FileState.IsValid());
		FSourceControlStateRef FileStateRef = FileState.ToSharedRef();
		FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(FileStateRef->GetFilename());

		check(AssetDataEntry != nullptr);
		++CurrentAsyncTask;
		AssetDataEntry->FetchAssetDataTask = Async(EAsyncExecution::TaskGraph,
												   [FileStateRef]() { ISourceControlModule::Get().GetAssetDataCache().FetchAssetData(FileStateRef); },
												   []() { ISourceControlModule::Get().GetAssetDataCache().OnFetchAssetDataComplete(); });
	}
}

void FSourceControlAssetDataCache::FetchAssetData(FSourceControlStateRef InFileState)
{
	if (FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(InFileState->GetFilename()))
	{
		check(!AssetDataEntry->bInitialized);
		check(AssetDataEntry->AssetDataArrayPtr.IsValid());

		if (InFileState->GetHistorySize() > 0)
		{
			if (TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = InFileState->GetHistoryItem(0))
			{
				const int64 MaxFetchSize = (1 << 20); // 1MB
				const bool bShouldGetFile = (MaxFetchSize < 0 || MaxFetchSize > static_cast<int64>(Revision->GetFileSize()));
				if (bShouldGetFile)
				{
					FString TempFileName;
					if (Revision->Get(TempFileName, EConcurrency::Asynchronous))
					{
						FAssetDataToLoad ToLoad = { TempFileName, InFileState->GetFilename() };
						AssetDataToLoad.Enqueue(MoveTemp(ToLoad));
					}
				}
			}
		}
	}
}

void FSourceControlAssetDataCache::OnFetchAssetDataComplete()
{
	--CurrentAsyncTask;
}

void FSourceControlAssetDataCache::UpdatePendingAssetData()
{
	FAssetDataToLoad ToLoad;

	while (AssetDataToLoad.Dequeue(ToLoad))
	{
		FSourceControlAssetDataEntry* AssetDataEntry = AssetDataCache.Find(ToLoad.Filename);

		check(AssetDataEntry != nullptr);
		USourceControlHelpers::GetAssetData(ToLoad.TempFilename, *(AssetDataEntry->AssetDataArrayPtr), nullptr);

		// We either already have AssetData or we can't go further with this asset.
		AssetDataEntry->bInitialized = true;
	}
}

void FSourceControlAssetDataCache::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	ClearPendingTasks();
}

void FSourceControlAssetDataCache::ClearPendingTasks()
{
	TArray<FString> EntriesToRemove;

	// Wait for tasks to stop
	for (auto& Pair : AssetDataCache)
	{
		FAssetDataCache::KeyType& Filename = Pair.Key;
		FAssetDataCache::ValueType& AssetDataEntry = Pair.Value;

		if (AssetDataEntry.bInitialized)
		{
			continue;
		}

		AssetDataEntry.FetchAssetDataTask.Wait();
		AssetDataEntry.FetchAssetDataTask.Reset();
		EntriesToRemove.Add(Filename);
	}

	// We remove all previously pending entries, they will restart the fetching process if needed again
	for (const FString& Entry : EntriesToRemove)
	{
		AssetDataCache.Remove(Entry);
	}

	// Clear queues
	FileHistoryToUpdate.Reset();
	FilesToDownload.Reset();
	AssetDataToFetch.Empty();
	AssetDataToLoad.Empty();
}
