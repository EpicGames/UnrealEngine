// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlCVars.h"

namespace SourceControlCVars
{
	TAutoConsoleVariable<bool> CVarSourceControlEnableRevertFromSceneOutliner(
		TEXT("SourceControl.Revert.EnableFromSceneOutliner"),
		false,
		TEXT("Allows a SourceControl 'Revert' operation to be triggered from the SceneOutliner."));

	TAutoConsoleVariable<bool> CVarSourceControlEnableRevertFromSubmitWidget(
		TEXT("SourceControl.Revert.EnableFromSubmitWidget"),
		false,
		TEXT("Allows a SourceControl 'Revert' operation to be triggered from the SubmitWidget."));

	TAutoConsoleVariable<bool> CVarSourceControlEnableRevertUnsaved(
		TEXT("SourceControl.RevertUnsaved.Enable"),
		false,
		TEXT("Allows a SourceControl 'Revert' operation to be triggered on an unsaved asset."));

	TAutoConsoleVariable<bool> CVarSourceControlEnableLoginDialogModal(
		TEXT("SourceControl.LoginDialog.ForceModal"),
		false,
		TEXT("Forces the SourceControl 'Login Dialog' to always be a modal dialog."));

	TAutoConsoleVariable<bool> CVarSourceControlEnableWorldReload(
		TEXT("SourceControl.EnableWorldReload"),
		false,
		TEXT("Allows a SourceControl operation to perform a world reload."));

	TAutoConsoleVariable<int32> CVarSourceControlIntervalBatchStatusUpdate(
		TEXT("SourceControl.IntervalBatchStatusUpdate"),
		1,
		TEXT("Controls interval that allows QueueStatusUpdate requests to be batched with others."));

	TAutoConsoleVariable<bool> CVarSourceControlDisplaySyncStatus(
		TEXT("SourceControl.DisplaySyncStatus"),
		false,
		TEXT("Allows SourceControl to display the Sync status.")
	);

	TAutoConsoleVariable<bool> CVarSourceControlDisplayCheckInStatus(
		TEXT("SourceControl.DisplayCheckInStatus"),
		false,
		TEXT("Allows SourceControl to display the CheckIn status.")
	);
}