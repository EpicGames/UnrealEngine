// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"

namespace SourceControlCVars
{
	extern TAutoConsoleVariable<bool> CVarSourceControlEnableRevertFromSceneOutliner;
	extern TAutoConsoleVariable<bool> CVarSourceControlEnableRevertFromSubmitWidget;
	extern TAutoConsoleVariable<bool> CVarSourceControlEnableRevertUnsaved;
	extern TAutoConsoleVariable<bool> CVarSourceControlEnableLoginDialogModal;

	extern TAutoConsoleVariable<int32> CVarSourceControlIntervalBatchStatusUpdate;

	extern TAutoConsoleVariable<bool> CVarSourceControlDisplaySyncStatus;
	extern TAutoConsoleVariable<bool> CVarSourceControlDisplayCheckInStatus;
}