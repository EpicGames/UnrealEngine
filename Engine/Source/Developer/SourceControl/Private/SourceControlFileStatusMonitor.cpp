// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlFileStatusMonitor.h"

#include "HAL/PlatformTime.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "Math/NumericLimits.h"
#include "Misc/App.h"
#include "Misc/ScopeExit.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "SourceControlOperations.h"

#if SOURCE_CONTROL_WITH_SLATE
#include "Framework/Application/SlateApplication.h"
#endif //#if SOURCE_CONTROL_WITH_SLATE

#if WITH_EDITORONLY_DATA
#include "Misc/PackageName.h"
#include "UObject/Package.h"
#endif // WITH_EDITORONLY_DATA

namespace
{

static bool CanQuerySourceControl(const FString& AbsPathname)
{
#if WITH_EDITORONLY_DATA
	FString PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(AbsPathname, PackageName))
	{
		if (UPackage* Package = FindPackage(nullptr, *PackageName))
		{
			// There's nothing to query if the package didn't come from disk or hasn't yet been saved to disk.
			return Package->GetHasBeenEndLoaded();
		}
	}
#endif // WITH_EDITORONLY_DATA

	return true;
}

}

FSourceControlFileStatusMonitor::FSourceControlFileStatusMonitor()
: ProbationPeriodPolicy(FTimespan::FromSeconds(1))
, RemainingPeriodPolicy(FTimespan::FromSeconds(5))
, RefreshPeriodPolicy(FTimespan::FromMinutes(5))
{
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FSourceControlFileStatusMonitor::Tick));
	SetSuspendMonitoringPolicy([]()
	{
#if SOURCE_CONTROL_WITH_SLATE
		if (!FApp::IsUnattended() && FSlateApplication::IsInitialized())
		{
			// By default, suspend monitoring if the user didn't interact for the last 5 minutes.
			return FPlatformTime::Seconds() - FSlateApplication::Get().GetLastUserInteractionTime() > FTimespan::FromMinutes(5).GetTotalSeconds();
		}
#endif //SOURCE_CONTROL_WITH_SLATE

		// Without slate there is not user interaction, so we always suspend the monitoring
		return true;
	});
}

FSourceControlFileStatusMonitor::~FSourceControlFileStatusMonitor()
{
	FTSTicker::RemoveTicker(TickerHandle);
}

TSharedPtr<FSourceControlFileStatusMonitor::FSourceControlFileStatus> FSourceControlFileStatusMonitor::FindFileStatus(const FString& AbsPathname) const
{
	if (const TSharedPtr<FSourceControlFileStatus>* FileStatus = MonitoredFiles.Find(AbsPathname))
	{
		return *FileStatus;
	}
	return nullptr;
}

void FSourceControlFileStatusMonitor::OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider)
{
	ensure(IsInGameThread()); // Concurrency issues if invoked from a background thread.
		
	// Start new with the new provider.
	RequestedStatusFiles.Reset();
	for (const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Pair : MonitoredFiles)
	{
		if (Pair.Value->FileState)
		{
			for (TPair<uintptr_t, FOnSourceControlFileStatus>& OwnerDelegatePair : Pair.Value->OwnerDelegateMap)
			{
				OwnerDelegatePair.Value.ExecuteIfBound(Pair.Key, nullptr); // Passing 'nullptr' state means that the state is now unknown.
			}
			Pair.Value->LastStatusCheckTimestampSecs = 0.0;
			Pair.Value->FileState.Reset();
		}
	}
	NewAddedFileCount = MonitoredFiles.Num();
	LastAddedFileTimeSecs = FPlatformTime::Seconds();
	OldestFileStatusTimeSecs = 0.0;
}

void FSourceControlFileStatusMonitor::StartMonitoringFile(uintptr_t OwnerId, const FString& AbsPathname, FOnSourceControlFileStatus OnSourceControlFileStatus)
{
	ensure(IsInGameThread()); // Concurrency issues if invoked from a background thread.

	// If the file is already monitored.
	if (TSharedPtr<FSourceControlFileStatus> FileStatus = FindFileStatus(AbsPathname))
	{
		// Another 'client' is looking at this file (if the client already exit, override the callback with the new one).
		FOnSourceControlFileStatus& OnSourceControlFileStatusDelegate = FileStatus->OwnerDelegateMap.FindOrAdd(OwnerId);
		OnSourceControlFileStatusDelegate = OnSourceControlFileStatus;

		// The monitor already knows the status.
		if (FileStatus->FileState)
		{
			OnSourceControlFileStatusDelegate.ExecuteIfBound(AbsPathname, FileStatus->FileState.Get());
		}
	}
	else
	{
		MonitoredFiles.Emplace(AbsPathname, MakeShared<FSourceControlFileStatus>(OwnerId, MoveTemp(OnSourceControlFileStatus)));
		LastAddedFileTimeSecs = FPlatformTime::Seconds();
		++NewAddedFileCount;
	}

	if (!SourceControlProviderChangedDelegateHandle.IsValid())
	{
		SourceControlProviderChangedDelegateHandle = ISourceControlModule::Get().RegisterProviderChanged(
			FSourceControlProviderChanged::FDelegate::CreateSP(this, &FSourceControlFileStatusMonitor::OnSourceControlProviderChanged));
	}
}

void FSourceControlFileStatusMonitor::StartMonitoringFiles(uintptr_t OwnerId, const TArray<FString>& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus)
{
	for (const FString& AbsPathname : AbsPathnames)
	{
		StartMonitoringFile(OwnerId, AbsPathname, OnSourceControlledFileStatus);
	}
}

void FSourceControlFileStatusMonitor::StartMonitoringFiles(uintptr_t OwnerId, const TSet<FString>& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus)
{
	for (const FString& AbsPathname : AbsPathnames)
	{
		StartMonitoringFile(OwnerId, AbsPathname, OnSourceControlledFileStatus);
	}
}

void FSourceControlFileStatusMonitor::StopMonitoringFile(uintptr_t OwnerId, const FString& AbsPathname)
{
	ensure(IsInGameThread()); // Concurrency issues if the callback in invoked from a background thread.

	if (TSharedPtr<FSourceControlFileStatus> FileStatus = FindFileStatus(AbsPathname))
	{
		if (FileStatus->OwnerDelegateMap.Remove(OwnerId) > 0 && FileStatus->OwnerDelegateMap.IsEmpty())
		{
			MonitoredFiles.Remove(AbsPathname);
		}
	}
}

void FSourceControlFileStatusMonitor::StopMonitoringFiles(uintptr_t OwnerId, const TArray<FString>& AbsPathnames)
{
	for (const FString& AbsPathname : AbsPathnames)
	{
		StopMonitoringFile(OwnerId, AbsPathname);
	}
}

void FSourceControlFileStatusMonitor::StopMonitoringFiles(uintptr_t OwnerId, const TSet<FString>& AbsPathnames)
{
	for (const FString& AbsPathname : AbsPathnames)
	{
		StopMonitoringFile(OwnerId, AbsPathname);
	}
}

void FSourceControlFileStatusMonitor::StopMonitoringFiles(uintptr_t OwnerId)
{
	ensure(IsInGameThread()); // Concurrency issues if the callback in invoked from a background thread.

	for (auto It = MonitoredFiles.CreateIterator(); It; ++It)
	{
		if (It->Value->OwnerDelegateMap.Remove(OwnerId) > 0 && It->Value->OwnerDelegateMap.IsEmpty())
		{
			It.RemoveCurrent();
		}
	}
}

void FSourceControlFileStatusMonitor::SetMonitoringFiles(uintptr_t OwnerId, TSet<FString>&& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus)
{
	for (auto It = MonitoredFiles.CreateIterator(); It; ++It)
	{
		const FString& MonitoredAbsPathname = It->Key;
		TSharedPtr<FSourceControlFileStatus>& MonitoredFileInfo = It->Value;
		FOnSourceControlFileStatus* OwnerCallbackDelegate = MonitoredFileInfo->OwnerDelegateMap.Find(OwnerId);

		// If the caller wants to monitor that file.
		if (AbsPathnames.Contains(MonitoredAbsPathname))
		{
			// If the caller was already monitoring the file.
			if (OwnerCallbackDelegate)
			{
				// Replace the delegate with the new one.
				*OwnerCallbackDelegate = OnSourceControlledFileStatus;
			}
			else // The file was monitored, but not by this caller. Add this caller.
			{
				MonitoredFileInfo->OwnerDelegateMap.Add(OwnerId, OnSourceControlledFileStatus);

				// If the status is already known.
				if (MonitoredFileInfo->FileState)
				{
					OnSourceControlledFileStatus.ExecuteIfBound(MonitoredAbsPathname, MonitoredFileInfo->FileState.Get());
				}
			}

			// That file was resolved, remove it from the set.
			AbsPathnames.Remove(MonitoredAbsPathname);
		}
		else if (OwnerCallbackDelegate) // The caller used to monitor this file but doesn't want to monitor it anymore
		{
			if (MonitoredFileInfo->OwnerDelegateMap.Num() > 1)
			{
				// Just remove the caller, leaving the others.
				MonitoredFileInfo->OwnerDelegateMap.Remove(OwnerId);
			}
			else
			{
				// Remove the caller and the files since nobody else monitor the file.
				It.RemoveCurrent();
			}
		}
	}

	// What remains in the set are the files that weren't monitored yet, start monitoring them.
	for (const FString& AbsPathname : AbsPathnames)
	{
		StartMonitoringFile(OwnerId, AbsPathname, OnSourceControlledFileStatus);
	}
}

TSet<FString> FSourceControlFileStatusMonitor::GetMonitoredFiles(uintptr_t OwnerId)
{
	TSet<FString> Pathnames;
	for (const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Pair : MonitoredFiles)
	{
		if (!Pair.Value->OwnerDelegateMap.Contains(OwnerId))
		{
			Pathnames.Add(Pair.Key);
		}
	}
	return Pathnames;
}

TOptional<FTimespan> FSourceControlFileStatusMonitor::GetStatusAge(const FString& AbsPathname) const
{
	TOptional<FTimespan> Age;
	if (TSharedPtr<FSourceControlFileStatus> FileStatus = FindFileStatus(AbsPathname))
	{
		Age.Emplace(FTimespan::FromSeconds(FileStatus->FileState ? FPlatformTime::Seconds() - FileStatus->LastStatusCheckTimestampSecs : 0.0));
	}
	return Age;
}

bool FSourceControlFileStatusMonitor::Tick(float DeltaTime)
{
	ensure(IsInGameThread()); // Concurrency issues if the callback in invoked from a background thread.

	// Nothing to check or a request is already in-flight.
	if (!ISourceControlModule::Get().IsEnabled() || !ISourceControlModule::Get().GetProvider().IsAvailable() || MonitoredFiles.IsEmpty() || HasOngoingRequest())
	{
		return true;
	}

	// Check if the monitor is suspended, not allowed to send requests.
	if (SuspendMonitoringPolicy && SuspendMonitoringPolicy())
	{
		return true;
	}

	const double NowSecs = FPlatformTime::Seconds();

	// Throttle the source control status check when no new files need to be checked. Don't overload the source control server with too many requests.
	if (NewAddedFileCount == 0 && NowSecs - OldestFileStatusTimeSecs < RefreshPeriodPolicy.GetTotalSeconds() && LastRemainingFileTimeSecs == 0.f)
	{
		return true; // Nothing to do this time around.
	}

	// Throttle the checks when remaining files are left.
	if (NewAddedFileCount == 0 && NowSecs - LastRemainingFileTimeSecs < RemainingPeriodPolicy.GetTotalSeconds() && LastRemainingFileTimeSecs != 0.f)
	{
		return true; // Delay the retry.
	}

	// Throttle the checks when new files are added. Batch new files edited close in time to be more efficient.
	if (NowSecs - LastAddedFileTimeSecs < ProbationPeriodPolicy.GetTotalSeconds() && NewAddedFileCount < MaxFileNumPerRequestPolicy)
	{
		return true; // Delay the request, give chance to capture more files.
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlFileStatusMonitor::Tick);

	TArray<const TPair<FString, TSharedPtr<FSourceControlFileStatus>>*> NewFiles;
	NewFiles.Reserve(NewAddedFileCount);

	TArray<const TPair<FString, TSharedPtr<FSourceControlFileStatus>>*> RefreshedFiles;
	RefreshedFiles.Reserve(MonitoredFiles.Num());

	// List all the files that are new and those that weren't updated recently.
	for (const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Pair : MonitoredFiles)
	{
		if (!Pair.Value->FileState.IsValid())
		{
			NewFiles.Add(&Pair);
		}
		else if (NowSecs - Pair.Value->LastStatusCheckTimestampSecs >= RefreshPeriodPolicy.GetTotalSeconds())
		{
			RefreshedFiles.Add(&Pair);
		}
	}

	// Too many status to query/refresh?
	if (NewFiles.Num() < MaxFileNumPerRequestPolicy && NewFiles.Num() + RefreshedFiles.Num() > MaxFileNumPerRequestPolicy)
	{
		// Get the status of all newly added files and refresh the ones that were less recently updated.
		RefreshedFiles.Sort([](const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Lhs, const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Rhs)
			{
				// Sort ascending as we are going to use Last()/Pop() later, so the oldest must be at the end.
				return Lhs.Value->LastStatusCheckTimestampSecs > Rhs.Value->LastStatusCheckTimestampSecs;
			});
	}

	uint32 NewFilesRemaining = NewFiles.Num();
	uint32 RefreshedFilesRemaining = RefreshedFiles.Num();

	RequestedStatusFiles.Reserve(RequestedStatusFiles.Num() + NewFiles.Num() + RefreshedFiles.Num());
	while (RequestedStatusFiles.Num() < MaxFileNumPerRequestPolicy)
	{
		if (NewFiles.Num() > 0)
		{
			const FString& AbsPathname = NewFiles.Last()->Key;
			if (CanQuerySourceControl(AbsPathname))
			{
				RequestedStatusFiles.Emplace(AbsPathname);
				NewFilesRemaining--;
			}
			NewFiles.Pop(EAllowShrinking::No);
		}
		else if (RefreshedFiles.Num())
		{
			const FString& AbsPathname = RefreshedFiles.Last()->Key;
			if (CanQuerySourceControl(AbsPathname))
			{
				RequestedStatusFiles.Emplace(AbsPathname);
				RefreshedFilesRemaining--;
			}
			RefreshedFiles.Pop(EAllowShrinking::No);
		}
		else
		{
			break; // All files to query/refresh were added.
		}
	}

	// The remaining number of new files.
	NewAddedFileCount = NewFiles.Num();
	LastRemainingFileTimeSecs = NewFilesRemaining > 0 ? FPlatformTime::Seconds() : 0.f;

	if (RequestedStatusFiles.Num())
	{
		LastSourceControlCheckSecs = NowSecs;

		TSharedRef<FUpdateStatus> UpdateStatusRequest = ISourceControlOperation::Create<FUpdateStatus>();

		ISourceControlModule::Get().GetProvider().Execute(UpdateStatusRequest, RequestedStatusFiles, EConcurrency::Asynchronous,
			FSourceControlOperationComplete::CreateSP(this, &FSourceControlFileStatusMonitor::OnSourceControlStatusUpdate));
	}

	return true;
}

void FSourceControlFileStatusMonitor::OnSourceControlStatusUpdate(const TSharedRef<ISourceControlOperation>& InOperation, ECommandResult::Type InResult)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FSourceControlFileStatusMonitor::OnSourceControlStatusUpdate);

	ensure(IsInGameThread()); // Concurrency issues if the callback in invoked from a background thread.

	const double NowSecs = FPlatformTime::Seconds();

	ON_SCOPE_EXIT
	{
		RequestedStatusFiles.Reset();

		OldestFileStatusTimeSecs = NowSecs;
		for (const TPair<FString, TSharedPtr<FSourceControlFileStatus>>& Pair : MonitoredFiles)
		{
			OldestFileStatusTimeSecs = FMath::Min(OldestFileStatusTimeSecs, Pair.Value->LastStatusCheckTimestampSecs);
		}
	};

	if (InResult != ECommandResult::Succeeded)
	{
		return;
	}

	for (const FString& AbsPathname : RequestedStatusFiles)
	{
		// NOTE: The file and its status can be removed while exeucting OnFileStatusUpdateDelegate. Keep the shared pointer to avoid early destruction.
		if (TSharedPtr<FSourceControlFileStatus> FileStatus = FindFileStatus(AbsPathname))
		{
			if (TSharedPtr<ISourceControlState> FileState = ISourceControlModule::Get().GetProvider().GetState(AbsPathname, EStateCacheUsage::Use))
			{
				FileStatus->FileState = MoveTemp(FileState);
				FileStatus->LastStatusCheckTimestampSecs = NowSecs;

				for (TPair<uintptr_t, FOnSourceControlFileStatus>& OwnerDelegatePair : FileStatus->OwnerDelegateMap)
				{
					OwnerDelegatePair.Value.ExecuteIfBound(AbsPathname, FileStatus->FileState.Get());
				}
			}
		}
	}
}
