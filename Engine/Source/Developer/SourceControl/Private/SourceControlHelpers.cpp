// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlHelpers.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "ISourceControlState.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Misc/ConfigContext.h"
#include "Misc/ScopedSlowTask.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "ISourceControlLabel.h"
#include "UObject/Linker.h"
#include "UObject/Package.h"
#include "UObject/UObjectIterator.h"
#include "Misc/PackageName.h"
#include "Logging/MessageLog.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Internationalization/PackageLocalizationUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceControlHelpers)

#if WITH_EDITOR
#include "Editor.h"
#include "EngineAnalytics.h"
#include "PackageTools.h"
#include "ObjectTools.h"
#include "FileHelpers.h"
#include "Bookmarks/BookmarkScoped.h"
#include "Interfaces/IAnalyticsProvider.h"
#endif

#define LOCTEXT_NAMESPACE "SourceControlHelpers"

namespace SourceControlHelpersInternal
{

/*
 * Status info set by LogError() and USourceControlHelpers methods if an error occurs
 * regardless whether their bSilent is set or not.
 * Should be empty if there is was no error.
 * @see	USourceControlHelpers::LastErrorMsg(), LogError()
 */
FText LastErrorText;

/* Store error and write to Log if bSilent is false. */
inline void LogError(const FText& ErrorText, bool bSilent)
{
	LastErrorText = ErrorText;

	if (!bSilent)
	{
		FMessageLog("SourceControl").Error(LastErrorText);
	}
}

/* Return provider if ready to go, else return nullptr. */
ISourceControlProvider* VerifySourceControl(bool bSilent)
{
	ISourceControlModule& SCModule = ISourceControlModule::Get();

	if (!SCModule.IsEnabled())
	{
		LogError(LOCTEXT("SourceControlDisabled", "Revision control is not enabled."), bSilent);

		return nullptr;
	}

	ISourceControlProvider* Provider = &SCModule.GetProvider();

	if (!Provider->IsAvailable())
	{
		LogError(LOCTEXT("SourceControlServerUnavailable", "Revision control server is currently not available."), bSilent);

		return nullptr;
	}

	// Clear the last error text if there hasn't been an error (yet).
	LastErrorText = FText::GetEmpty();

	return Provider;
}


/*
 * Converts specified file to fully qualified file path that is compatible with source control.
 *
 * @param	InFile		File string - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
 * @param	bSilent		if false then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
 * @return	Fully qualified file path to use with source control or "" if conversion unsuccessful.
 */
FString ConvertFileToQualifiedPath(const FString& InFile, bool bSilent, bool bAllowDirectories = false, const TCHAR* AssociatedExtension = nullptr)
{
	// Converted to qualified file path
	FString SCFile;

	if (InFile.IsEmpty())
	{
		LogError(LOCTEXT("UnspecifiedFile", "File not specified"), bSilent);

		return SCFile;
	}

	// Try to determine if file is one of:
	// - fully qualified path
	// - relative path
	// - long package name
	// - asset path
	// - export text path (often stored on clipboard)
	//
	// For example:
	// - D:\Epic\Dev-Ent\Projects\Python3rdBP\Content\Mannequin\Animations\ThirdPersonIdle.uasset
	// - Content\Mannequin\Animations\ThirdPersonIdle.uasset
	// - /Game/Mannequin/Animations/ThirdPersonIdle
	// - /Game/Mannequin/Animations/ThirdPersonIdle.ThirdPersonIdle
	// - AnimSequence'/Game/Mannequin/Animations/ThirdPersonIdle.ThirdPersonIdle'

	SCFile = InFile;

	// Is ExportTextPath (often stored in Clipboard) form?
	//  - i.e. AnimSequence'/Game/Mannequin/Animations/ThirdPersonIdle.ThirdPersonIdle'
	if (SCFile[SCFile.Len() - 1] == '\'')
	{
		SCFile = FPackageName::ExportTextPathToObjectPath(SCFile);
	}

	// Package paths
	if (SCFile[0] == TEXT('/') && FPackageName::IsValidLongPackageName(SCFile, /*bIncludeReadOnlyRoots*/false))
	{
		// Assume it is a package
		bool bPackage = true;

		// Try to get filename by finding it on disk
		if (!FPackageName::DoesPackageExist(SCFile, &SCFile))
		{
			// First do the conversion without any extension set, as this will allow us to test whether the path represents an existing directory rather than an asset
			if (FPackageName::TryConvertLongPackageNameToFilename(SCFile, SCFile))
			{
				if (bAllowDirectories && FPaths::DirectoryExists(SCFile))
				{
					// This path mapped to a known directory, so ensure it ends in a slash
					SCFile /= FString();
				}
				else if (AssociatedExtension)
				{
					// Just use the requested extension
					SCFile += AssociatedExtension;
				}
				else
				{
					// The package does not exist on disk, see if we can find it in memory and predict the file extension
					UPackage* Package = FindPackage(nullptr, *SCFile);
					SCFile += (Package && Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension());
				}
			}
			else
			{
				bPackage = false;
			}
		}

		if (bPackage)
		{
			SCFile = FPaths::ConvertRelativePathToFull(SCFile);

			return SCFile;
		}
	}

	// Assume it is a qualified or relative file path

	// Could normalize it
	//FPaths::NormalizeFilename(SCFile);

	if (!FPaths::IsRelative(SCFile))
	{
		return SCFile;
	}

	// Qualify based on process base directory.
	// Something akin to "C:/Epic/UE/Engine/Binaries/Win64/" as a current path.
	SCFile = FPaths::ConvertRelativePathToFull(InFile);

	if (FPaths::FileExists(SCFile) || (bAllowDirectories && FPaths::DirectoryExists(SCFile)))
	{
		return SCFile;
	}

	// Qualify based on project directory.
	SCFile = FPaths::ConvertRelativePathToFull(FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), InFile);

	if (FPaths::FileExists(SCFile) || (bAllowDirectories && FPaths::DirectoryExists(SCFile)))
	{
		return SCFile;
	}

	// Qualify based on Engine directory
	SCFile = FPaths::ConvertRelativePathToFull(FPaths::ConvertRelativePathToFull(FPaths::EngineDir()), InFile);

	return SCFile;
}


/**
 * Converts specified files to fully qualified file paths that are compatible with source control.
 *
 * @param	InFiles			File strings - can be either fully qualified path, relative path, long package name, asset name or export text path (often stored on clipboard)
 * @param	OutFilePaths	Fully qualified file paths to use with source control or "" if conversion unsuccessful.
 * @param	bSilent			if false then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
 * @return	true if all files successfully converted, false if any had errors
 */
bool ConvertFilesToQualifiedPaths(const TArray<FString>& InFiles, TArray<FString>& OutFilePaths, bool bSilent, bool bAllowDirectories = false)
{
	uint32 SkipNum = 0u;

	for (const FString& File : InFiles)
	{
		FString SCFile = ConvertFileToQualifiedPath(File, bSilent, bAllowDirectories);

		if (SCFile.IsEmpty())
		{
			SkipNum++;
		}
		else
		{
			OutFilePaths.Add(MoveTemp(SCFile));
		}
	}

	if (SkipNum)
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("SkipNum"), FText::AsNumber(SkipNum));
		LogError(FText::Format(LOCTEXT("FilesSkipped", "During conversion to qualified file paths, {SkipNum} files were skipped!"), Arguments), bSilent);

		return false;
	}

	return true;
}

void RunQueryFileStates(TConstArrayView<FString> InFiles, EConcurrency::Type InConcurrencyType, bool bSilent, bool bUseSourceControlStateCache, TFunction<void(FSourceControlState)> FileStateCallback)
{
	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);
	if (!Provider)
	{
		return;
	}

	TArray<FString> SCFilesToUpdate;
	for (const FString& File : InFiles)
	{
		// Determine file type and ensure it is in form source control wants
		FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(File, bSilent);

		if (SCFile.IsEmpty())
		{
			// Improper or invalid SCC state
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("InFile"), FText::FromString(File));
			SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotResolveFilepath", "Could not resolve file path for '{InFile}'."), Arguments), bSilent);
			continue;
		}
		SCFilesToUpdate.Emplace(MoveTemp(SCFile));
	}

	auto SetStatusAndRunCallback = [](const TArray<FString>& InFilesToUpdate, const ISourceControlProvider* Provider, const TFunction<void(FSourceControlState)>& InCallbackFn)
	{
		TArray<FSourceControlStateRef> SCStates;
		const_cast<ISourceControlProvider*>(Provider)->GetState(InFilesToUpdate, SCStates, EStateCacheUsage::Use);

		for (const FSourceControlStatePtr SCState : SCStates)
		{
			FSourceControlState State;
			State.SetFromStatus(SCState);
			InCallbackFn(State);
		}
	};

	if (!bUseSourceControlStateCache)
	{
		// Make sure we update the modified state of the files (Perforce requires this
		// since can be a more expensive test).
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateModifiedState(true);

		ISourceControlModule::Get().GetProvider().Execute(
			UpdateStatusOperation,
			SCFilesToUpdate,
			InConcurrencyType,
			FSourceControlOperationComplete::CreateLambda([FileStateCallback, Provider, SCFilesToUpdate, SetStatusAndRunCallback](const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult)
			{
				SetStatusAndRunCallback(SCFilesToUpdate, Provider, FileStateCallback);
			}));
	}
	else
	{
		SetStatusAndRunCallback(SCFilesToUpdate, Provider, FileStateCallback);
	}
}

}  // namespace SourceControlHelpersInternal


FString USourceControlHelpers::CurrentProvider()
{
	// Note that if there is no provider there is still a dummy default provider object
	ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

	return Provider.GetName().ToString();
}


bool USourceControlHelpers::IsEnabled()
{
	return ISourceControlModule::Get().IsEnabled();
}


bool USourceControlHelpers::IsAvailable()
{
	ISourceControlModule& SCModule = ISourceControlModule::Get();

	return SCModule.IsEnabled() && SCModule.GetProvider().IsAvailable();
}


FText USourceControlHelpers::LastErrorMsg()
{
	return SourceControlHelpersInternal::LastErrorText;
}


bool USourceControlHelpers::SyncFile(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent, /*bAllowDirectories*/true);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	if (Provider->Execute(ISourceControlOperation::Create<FSync>(), SCFile) == ECommandResult::Succeeded)
	{
		return true;
	}

	// Only error info after this point

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
	Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));

	SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("SyncFailed", "Failed to sync file '{InFile}' ({SCFile})."), Arguments), bSilent);
	return false;
}


bool USourceControlHelpers::SyncFiles(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately 
	if (InFiles.IsEmpty())
	{
		return true;
	}

	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	TArray<FString> FilePaths;

	// Even if some files were skipped, still apply to the others
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, FilePaths, bSilent, /*bAllowDirectories*/true);

	// Less error checking and info is made for multiple files than the single file version.
	// This multi-file version could be made similarly more sophisticated.
	ECommandResult::Type Result = Provider->Execute(ISourceControlOperation::Create<FSync>(), FilePaths);

	return !bFilesSkipped && (Result == ECommandResult::Succeeded);
}

void LogCheckoutFailure(const FString& InFile, const FString& SCFile, FSourceControlStatePtr SCState, bool bCheckoutFailed, bool bSilent)
{
	FString SimultaneousCheckoutUser;
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
	Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));

	if (bCheckoutFailed)
	{
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CheckoutFailed", "Failed to check out file '{InFile}' ({SCFile})."), Arguments), bSilent);
	}
	else if (!SCState->IsSourceControlled())
	{
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("NotSourceControlled", "Could not check out the file '{InFile}' because it is not under revision control ({SCFile})."), Arguments), bSilent);
	}
	else if (!SCState->IsCurrent())
	{
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("NotAtHeadRevision", "File '{InFile}' is not at head revision ({SCFile})."), Arguments), bSilent);
	}
	else if (SCState->IsCheckedOutOther(&(SimultaneousCheckoutUser)))
	{
		Arguments.Add(TEXT("SimultaneousCheckoutUser"), FText::FromString(SimultaneousCheckoutUser));
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("SimultaneousCheckout", "File '{InFile}' is checked out by another ({SimultaneousCheckoutUser}) ({SCFile})."), Arguments), bSilent);
	}
	else
	{
		// Improper or invalid SCC state
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDetermineState", "Could not determine revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);
	}
}

bool USourceControlHelpers::CheckOutFile(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	FSourceControlStatePtr SCState = Provider->GetState(SCFile, EStateCacheUsage::ForceUpdate);

	if (!SCState.IsValid())
	{
		LogCheckoutFailure(InFile, SCFile, SCState, false, bSilent);
		return false;
	}

	if (SCState->IsCheckedOut() || SCState->IsAdded())
	{
		// Already checked out or opened for add
		return true;
	}

	bool bCheckOutFailed = false;

	if (SCState->CanCheckout())
	{
		if (Provider->Execute(ISourceControlOperation::Create<FCheckOut>(), SCFile) == ECommandResult::Succeeded)
		{
			return true;
		}

		bCheckOutFailed = true;
	}

	LogCheckoutFailure(InFile, SCFile, SCState, bCheckOutFailed, bSilent);

	return false;
}

bool USourceControlHelpers::CheckOutFiles(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	// Determine file type and ensure it is in form source control wants
	// Even if some files were skipped, still apply to the others
	TArray<FString> SCFiles;
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, SCFiles, bSilent);
	const int32 NumFiles = SCFiles.Num();

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);
	if (!Provider)
	{
		// Error or can't communicate with source control
		return false;
	}

	TArray<FSourceControlStateRef> SCStates;
	Provider->GetState(SCFiles, SCStates, EStateCacheUsage::ForceUpdate);

	TArray<FString> SCFilesToCheckout;
	bool bCannotCheckoutAtLeastOneFile = false;
	for (int32 Index = 0; Index < NumFiles; ++Index)
	{
		FString SCFile = SCFiles[Index];
		FSourceControlStateRef SCState = SCStates[Index];

		// Less error checking and info is made for multiple files than the single file version.
		// This multi-file version could be made similarly more sophisticated.
		if (!SCState->IsCheckedOut() && !SCState->IsAdded())
		{
			if (SCState->CanCheckout())
			{
				SCFilesToCheckout.Add(SCFile);
			}
			else
			{
				bCannotCheckoutAtLeastOneFile = true;
				LogCheckoutFailure(InFiles[Index], SCFile, SCState, false, bSilent);
			}
		}
	}

	bool bSuccess = !bFilesSkipped && !bCannotCheckoutAtLeastOneFile;
	if (bSuccess && SCFilesToCheckout.Num())
	{
		bSuccess = Provider->Execute(ISourceControlOperation::Create<FCheckOut>(), SCFilesToCheckout) == ECommandResult::Succeeded;
	}

	return bSuccess;
}


bool USourceControlHelpers::CheckOutOrAddFile(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	FSourceControlStatePtr SCState = Provider->GetState(SCFile, EStateCacheUsage::ForceUpdate);

	if (!SCState.IsValid())
	{
		// Improper or invalid SCC state
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
		Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDetermineState", "Could not determine revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);

		return false;
	}

	if (SCState->IsCheckedOut() || SCState->IsAdded())
	{
		// Already checked out or opened for add
		return true;
	}

	// Stuff single file in array for functions that require array
	TArray<FString> FilesToBeCheckedOut;
	FilesToBeCheckedOut.Add(SCFile);

	if (SCState->CanCheckout())
	{
		if (Provider->Execute(ISourceControlOperation::Create<FCheckOut>(), FilesToBeCheckedOut) != ECommandResult::Succeeded)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
			Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
			SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CheckoutFailed", "Failed to check out file '{InFile}' ({SCFile})."), Arguments), bSilent);

			return false;
		}

		return true;
	}

	bool bAddFail = false;

	if (!SCState->IsSourceControlled())
	{
		if (Provider->Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilesToBeCheckedOut) == ECommandResult::Succeeded)
		{
			return true;
		}

		bAddFail = true;;
	}

	FString SimultaneousCheckoutUser;
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
	Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));

	if (bAddFail)
	{
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("AddFailed", "Failed to add file '{InFile}' to revision control ({SCFile})."), Arguments), bSilent);
	}
	else if (!SCState->IsCurrent())
	{
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("NotAtHeadRevision", "File '{InFile}' is not at head revision ({SCFile})."), Arguments), bSilent);
	}
	else if (SCState->IsCheckedOutOther(&(SimultaneousCheckoutUser)))
	{
		Arguments.Add(TEXT("SimultaneousCheckoutUser"), FText::FromString(SimultaneousCheckoutUser));
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("SimultaneousCheckout", "File '{InFile}' is checked out by another ({SimultaneousCheckoutUser}) ({SCFile})."), Arguments), bSilent);
	}
	else
	{
		// Improper or invalid SCC state
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDetermineState", "Could not determine revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);
	}

	return false;
}

bool USourceControlHelpers::CheckOutOrAddFiles(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	// Determine file type and ensure it is in form source control wants
	// Even if some files were skipped, still apply to the others
	TArray<FString> SCFiles;
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, SCFiles, bSilent);
	const int32 NumFiles = SCFiles.Num();

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);
	if (!Provider)
	{
		// Error or can't communicate with source control
		return false;
	}

	TArray<FSourceControlStateRef> SCStates;
	Provider->GetState(SCFiles, SCStates, EStateCacheUsage::ForceUpdate);

	TArray<FString> SCFilesToAdd;
	TArray<FString> SCFilesToCheckout;
	bool bCannotAddAtLeastOneFile = false;
	bool bCannotCheckoutAtLeastOneFile = false;
	for (int32 Index = 0; Index < NumFiles; ++Index)
	{
		FString SCFile = SCFiles[Index];
		FSourceControlStateRef SCState = SCStates[Index];

		// Less error checking and info is made for multiple files than the single file version.
		// This multi-file version could be made similarly more sophisticated.
		if (!SCState->IsCheckedOut() && !SCState->IsAdded())
		{
			if (!SCState->IsSourceControlled())
			{
				if (SCState->CanAdd())
				{
					SCFilesToAdd.Add(SCFile);
				}
				else
				{
					bCannotAddAtLeastOneFile = true;
				}
			}
			else
			{
				if (SCState->CanCheckout())
				{
					SCFilesToCheckout.Add(SCFile);
				}
				else
				{
					bCannotCheckoutAtLeastOneFile = true;
				}
			}
		}
	}

	bool bSuccess = !bFilesSkipped && !bCannotCheckoutAtLeastOneFile && !bCannotAddAtLeastOneFile;
	if (SCFilesToAdd.Num())
	{
		bSuccess &= Provider->Execute(ISourceControlOperation::Create<FMarkForAdd>(), SCFilesToAdd) == ECommandResult::Succeeded;
	}

	if (SCFilesToCheckout.Num())
	{
		bSuccess &= Provider->Execute(ISourceControlOperation::Create<FCheckOut>(), SCFilesToCheckout) == ECommandResult::Succeeded;
	}

	return bSuccess;
}

bool USourceControlHelpers::MarkFileForAdd(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	// Mark for add now if needed
	FSourceControlStatePtr SCState = Provider->GetState(SCFile, EStateCacheUsage::Use);

	if (!SCState.IsValid())
	{
		// Improper or invalid SCC state
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
		Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDetermineState", "Could not determine revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);

		return false;
	}

	// Add if necessary
	if (SCState->IsUnknown() || (!SCState->IsSourceControlled() && !SCState->IsAdded()))
	{
		if (Provider->Execute(ISourceControlOperation::Create<FMarkForAdd>(), SCFile) != ECommandResult::Succeeded)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
			Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
			SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("MarkForAddFailed", "Failed to add file '{InFile}' to revision control ({SCFile})."), Arguments), bSilent);

			return false;
		}
	}

	return true;
}


bool USourceControlHelpers::MarkFilesForAdd(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	TArray<FString> FilePaths;

	// Even if some files were skipped, still apply to the others
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, FilePaths, bSilent);

	// Less error checking and info is made for multiple files than the single file version.
	// This multi-file version could be made similarly more sophisticated.
	ECommandResult::Type Result = Provider->Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilePaths);

	return !bFilesSkipped && (Result == ECommandResult::Succeeded);
}


bool USourceControlHelpers::MarkFileForDelete(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		// Error or can't communicate with source control
		// Could erase it anyway, though keeping it for now.
		return false;
	}

	FSourceControlStatePtr SCState = Provider->GetState(SCFile, EStateCacheUsage::ForceUpdate);

	if (!SCState.IsValid())
	{
		// Improper or invalid SCC state
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
		Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
		SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDetermineState", "Could not determine revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);

		return false;
	}

	if (SCState->IsSourceControlled())
	{
		bool bAdded = SCState->IsAdded();

		if (bAdded || SCState->IsCheckedOut())
		{
			if (Provider->Execute(ISourceControlOperation::Create<FRevert>(), SCFile) != ECommandResult::Succeeded)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
				Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
				SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotRevert", "Could not revert revision control state of file '{InFile}' ({SCFile})."), Arguments), bSilent);

				return false;
			}
		}

		if (!bAdded)
		{
			// Was previously added to source control so mark it for delete
			if (Provider->Execute(ISourceControlOperation::Create<FDelete>(), SCFile) != ECommandResult::Succeeded)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("InFile"), FText::FromString(InFile));
				Arguments.Add(TEXT("SCFile"), FText::FromString(SCFile));
				SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotDelete", "Could not delete file '{InFile}' from revision control ({SCFile})."), Arguments), bSilent);

				return false;
			}
		}
	}

	// Delete file if it still exists
	IFileManager& FileManager = IFileManager::Get();

	if (FileManager.FileExists(*SCFile))
	{
		// Just a regular file not tracked by source control so erase it.
		// Don't bother checking if it exists since Delete doesn't care.
		return FileManager.Delete(*SCFile, false, true);
	}
	else
	{
		return true; 
	}
}


bool USourceControlHelpers::MarkFilesForDelete(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	// Determine file type and ensure it is in form source control wants
	// Even if some files were skipped, still apply to the others
	TArray<FString> SCFiles;
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, SCFiles, bSilent);
	const int32 NumFiles = SCFiles.Num();

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);
	if (!Provider)
	{
		// Error or can't communicate with source control
		// Could erase the files anyway, though keeping them for now.
		return false;
	}

	TArray<FSourceControlStateRef> SCStates;
	Provider->GetState(SCFiles, SCStates, EStateCacheUsage::ForceUpdate);

	TArray<FString> SCFilesToRevert;
	TArray<FString> SCFilesToMarkForDelete;
	bool bCannotDeleteAtLeastOneFile = false;
	for (int32 Index = 0; Index < NumFiles; ++Index)
	{
		FString SCFile = SCFiles[Index];
		FSourceControlStateRef SCState = SCStates[Index];

		// Less error checking and info is made for multiple files than the single file version.
		// This multi-file version could be made similarly more sophisticated.
		if (SCState->IsSourceControlled())
		{
			bool bAdded = SCState->IsAdded();
			if (bAdded || SCState->IsCheckedOut())
			{
				SCFilesToRevert.Add(SCFile);
			}

			if (!bAdded)
			{
				if (SCState->CanDelete())
				{
					SCFilesToMarkForDelete.Add(SCFile);
				}
				else
				{
					bCannotDeleteAtLeastOneFile = true;
				}
			}
		}
	}

	bool bSuccess = !bFilesSkipped && !bCannotDeleteAtLeastOneFile;
	if (SCFilesToRevert.Num())
	{
		bSuccess &= Provider->Execute(ISourceControlOperation::Create<FRevert>(), SCFilesToRevert) == ECommandResult::Succeeded;
	}

	if (SCFilesToMarkForDelete.Num())
	{
		bSuccess &= Provider->Execute(ISourceControlOperation::Create<FDelete>(), SCFilesToMarkForDelete) == ECommandResult::Succeeded;
	}

	// Delete remaining files if they still exist : 
	IFileManager& FileManager = IFileManager::Get();
	for (FString SCFile : SCFiles)
	{
		if (FileManager.FileExists(*SCFile))
		{
			// Just a regular file not tracked by source control so erase it.
			// Don't bother checking if it exists since Delete doesn't care.
			bSuccess &= FileManager.Delete(*SCFile, false, true);
		}
	}

	return bSuccess;
}


bool USourceControlHelpers::RevertFile(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	// Revert file regardless of whether it has had any changes made
	ECommandResult::Type Result = Provider->Execute(ISourceControlOperation::Create<FRevert>(), SCFile);

	return Result == ECommandResult::Succeeded;
}

#if WITH_EDITOR
bool USourceControlHelpers::ApplyOperationAndReloadPackages(const TArray<FString>& InFilenames,	const TFunctionRef<bool(const TArray<FString>&)>& InOperation, bool bReloadWorld, bool bInteractive)
{
	const double TimeStarted = FPlatformTime::Seconds();

	double TimeTakenFlush = 0;
	double TimeTakenReload = 0;
	double TimeTakenOperation = 0;
	double TimeTakenUnload = 0;

	TArray<FString> PackageNames;
	TArray<FString> PackageFilenames;
	TArray<FString> FilteredPackages;
	TArray<FString> NotFoundPackages;
	TArray<UPackage*> UnlinkPackages;

	/** 
	 * This is necessary to cover an edge case where the user reverts a deleted Level file.
	 * In that case PackageFilename_Internal() will not be able to resolve to the correct extension
	 * as the file is deleted and the package cannot be inspected
	 */
	TMap<FString, FString> MapPackageNamesToFilenames;

	bool bSuccess = false;

	// Normalize packagenames and filenames
	for (const FString& Filename : InFilenames)
	{
		FString Result;
		
		if (FPackageName::TryConvertFilenameToLongPackageName(Filename, Result))
		{
			FString Extension = FPaths::GetExtension(Filename);
			if (Extension == TEXT("umap"))
			{
				MapPackageNamesToFilenames.Add(Result, Filename);
			}

			PackageNames.Add(MoveTemp(Result));
		}
		else
		{
			PackageNames.Add(Filename);
		}
	}

	// If bReloadWorld=false, remove packages if they are loaded external packages or world
	// If bReloadWorld=true, include world packages to reload
	TSet<UPackage*> UniqueLoadedPackages;
	PackageNames.RemoveAll([&FilteredPackages, &UniqueLoadedPackages, &NotFoundPackages, &UnlinkPackages, bReloadWorld](const FString& PackageName) -> bool
	{
		UPackage* Package = FindPackage(NULL, *PackageName);
		
		if (Package != nullptr)
		{
			if (UWorld* World = UWorld::FindWorldInPackage(Package))
			{
				if (!bReloadWorld)
				{
					FilteredPackages.Emplace(PackageName);
					return true; // remove the package
				}
			}
			else if (UObject* Asset = Package->FindAssetInPackage())
			{
				if (Asset->IsPackageExternal())
				{
					const bool bIsAssetPartOfWorld = Asset->GetWorld() && Asset->GetWorld()->GetPackage();
					// In case this is an explicit world object/actor reload or just an external object unload/detach it 
					if (bReloadWorld || !bIsAssetPartOfWorld)
					{
						// detach linker on the object
						UnlinkPackages.Emplace(Package);

						// track its world for reloading - not the object package itself
						if (bIsAssetPartOfWorld)
						{
							UniqueLoadedPackages.Add(Asset->GetWorld()->GetPackage());
						}

						return false;
					}
					else
					{
						FilteredPackages.Emplace(PackageName);
						return true; // remove the package
					}
				}
			}

			UniqueLoadedPackages.Add(Package);
		}
		else
		{
			NotFoundPackages.Add(PackageName);
		}

		return false; // do not remove the package
	});

	if (!FilteredPackages.IsEmpty())
	{
		TStringBuilder<2048> Builder;
		Builder.Join(FilteredPackages, TEXT(", "));
		const FString Packages = *Builder;

		UE_LOGF(LogSourceControl, Warning, "This operation could not complete on the following map or external packages, please unload them before retrying : %ls", *Packages);
		return false;
	}

	// Reverting may reintroduce some packages, so we need to ensure they're picked up...
	if (!NotFoundPackages.IsEmpty() && bReloadWorld)
	{
		const FString& ExternalActorsFolderName = FPackagePath::GetExternalActorsFolderName();
		const FString& ExternalObjectsFolderName = FPackagePath::GetExternalObjectsFolderName();

		TArray<FString> ExternalFolderNames;
		ExternalFolderNames.Add(ExternalActorsFolderName);
		ExternalFolderNames.Add(ExternalObjectsFolderName);

		// Gather the ExternalPaths in which we're reintroducing packages...
		TSet<FString> NotFoundExternalPaths;
		for (const FString& ExternalFolderName : ExternalFolderNames)
		{
			for (const FString& PackageName : NotFoundPackages)
			{
				int32 Index = PackageName.Find(ExternalFolderName);
				if (Index != INDEX_NONE)
				{
					// Format: /<MountPoint>/__External<xxxxx>__/<Level>/0/AB/CDEFGHIJKLMNOPQRSTUVWX
					// Result: /<MountPoint>/__External<xxxxx>__/<Level>

					Index += ExternalFolderName.Len();
					Index += 1;

					Index = PackageName.Find("/", ESearchCase::IgnoreCase, ESearchDir::FromStart, Index);
					if (Index != INDEX_NONE)
					{
						NotFoundExternalPaths.Add(PackageName.Left(Index));
					}
				}
			}
		}

		// See if any of the loaded levels stores their external actors in any of those paths...
		for (TObjectIterator<ULevel> LevelIt; LevelIt; ++LevelIt)
		{
			ULevel* Level = (*LevelIt);
			if (Level->IsUsingExternalActors())
			{
				const FString& ExternalActorPath = ULevel::GetExternalActorsPath(Level->GetPackage());
				if (NotFoundExternalPaths.Contains(ExternalActorPath))
				{
					UniqueLoadedPackages.Add(Level->GetWorld()->GetPackage());
				}
			}
			if (Level->IsUsingExternalObjects())
			{
				const TArray<FString> ExternalObjectPaths = ULevel::GetExternalObjectsPaths(Level->GetPackage()->GetName());
				for (const FString& ExternalObjectPath : ExternalObjectPaths)
				{
					if (NotFoundExternalPaths.Contains(ExternalObjectPath))
					{
						UniqueLoadedPackages.Add(Level->GetWorld()->GetPackage());
					}
				}
			}
		}
	}

	// Prepare the packages to be reverted...
	TArray<UPackage*> LoadedPackages = UniqueLoadedPackages.Array();

	// Make sure they are unlinked...
	UnlinkPackages.Append(LoadedPackages);
	if (UnlinkPackages.Num() > 0)
	{
		const double TimeStartedFlush = FPlatformTime::Seconds();

		TArray<UObject*> LoadedObjects;
		TArray<int32> PendingPackageRequestIds;
		LoadedObjects.Reserve(UnlinkPackages.Num());
		PendingPackageRequestIds.Reserve(UnlinkPackages.Num());
		for (UPackage* Package : UnlinkPackages)
		{
			LoadedObjects.Emplace(Package);

			if (!Package->IsFullyLoaded())
			{
				PendingPackageRequestIds.Emplace(LoadPackageAsync(Package->GetName()));
			}
		}

		FlushAsyncLoading(PendingPackageRequestIds);
		ResetLoaders(LoadedObjects);

		TimeTakenFlush = FPlatformTime::Seconds() - TimeStartedFlush;
	}

	for (int32 PackageIndex = 0; PackageIndex < PackageNames.Num(); PackageIndex++)
	{
		if (MapPackageNamesToFilenames.Contains(PackageNames[PackageIndex]))
		{
			PackageFilenames.Add(MapPackageNamesToFilenames[PackageNames[PackageIndex]]);
		}
		else
		{
			PackageFilenames.Add(PackageFilename(PackageNames[PackageIndex]));
		}
	}

	// Apply Operation
	{
		const double TimeStartedOperation = FPlatformTime::Seconds();

		bSuccess = InOperation(PackageFilenames);

		TimeTakenOperation = FPlatformTime::Seconds() - TimeStartedOperation;
	}

	// Reverting may have deleted some packages, so we need to delete those and unload them rather than re-load them...
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	TArray<TWeakObjectPtr<UObject>> ObjectsMissingOnDisk;
	LoadedPackages.RemoveAll([&](UPackage* InPackage) -> bool
		{
			const FString PackageExtension = InPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			const FString PackageFilename = FPackageName::LongPackageNameToFilename(InPackage->GetName(), PackageExtension);
			if (!FPaths::FileExists(PackageFilename))
			{
				TArray<FAssetData> Assets;
				AssetRegistryModule.Get().GetAssetsByPackageName(*InPackage->GetName(), Assets);

				for (const FAssetData& Asset : Assets)
				{
					if (UObject* ObjectToDelete = Asset.FastGetAsset())
					{
						ObjectsMissingOnDisk.Add(ObjectToDelete);
					}
				}
				return true; // remove package
			}
			return false; // keep package
		});

	// Hot-reload the new packages...
	if (LoadedPackages.Num() > 0)
	{
		const double TimeStartedReload = FPlatformTime::Seconds();

		FBookmarkScoped BookmarkScoped;
		FScopedSlowTask SlowTaskScoped(1.f, NSLOCTEXT("SourceControl", "Reload", "Reloading..."));
		SlowTaskScoped.MakeDialog();
		SlowTaskScoped.EnterProgressFrame(1.f);
		SlowTaskScoped.ForceRefresh();

		// Split into world and non-world packages and reload them separately.
		TArray<UPackage*> LoadedWorldPackages;
		TArray<UPackage*> LoadedNonWorldPackages;
		LoadedWorldPackages.Reserve(LoadedPackages.Num());
		LoadedNonWorldPackages.Reserve(LoadedPackages.Num());

		for (UPackage* Package : LoadedPackages)
		{
			if (UWorld::FindWorldInPackage(Package))
			{
				LoadedWorldPackages.Add(Package);
			}
			else
			{
				LoadedNonWorldPackages.Add(Package);
			}
		}

		// Reload non world package(s).
		if (LoadedNonWorldPackages.Num() > 0)
		{
			FText OutReloadErrorMsg;
			UPackageTools::ReloadPackages(LoadedNonWorldPackages, OutReloadErrorMsg, bInteractive ? EReloadPackagesInteractionMode::Interactive : EReloadPackagesInteractionMode::AssumePositive);
			if (!OutReloadErrorMsg.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Warning, "%ls", *OutReloadErrorMsg.ToString());
			}
		}

		// Reload world package(s).
		if (LoadedWorldPackages.Num() > 0)
		{
			FText OutReloadErrorMsg;
			UPackageTools::ReloadPackages(LoadedWorldPackages, OutReloadErrorMsg, bInteractive ? EReloadPackagesInteractionMode::Interactive : EReloadPackagesInteractionMode::AssumePositive);
			if (!OutReloadErrorMsg.IsEmpty())
			{
				UE_LOGF(LogSourceControl, Warning, "%ls", *OutReloadErrorMsg.ToString());
			}
		}

		TimeTakenReload = FPlatformTime::Seconds() - TimeStartedReload;
	}

	// A world reload might have already deleted some objects, so check which missing ones are still valid.
	TArray<UObject*> ObjectsToDelete;
	if (ObjectsMissingOnDisk.Num() > 0)
	{
		for (TWeakObjectPtr<UObject> Object : ObjectsMissingOnDisk)
		{
			if (UObject* ObjectToDelete = Object.Get())
			{
				ObjectsToDelete.Add(ObjectToDelete);
			}
		}
	}

	// Delete and Unload assets...
	if (ObjectsToDelete.Num() > 0)
	{
		const double TimeStartedUnload = FPlatformTime::Seconds();

		if (ObjectTools::DeleteObjectsUnchecked(ObjectsToDelete) != ObjectsToDelete.Num())
		{
			UE_LOGF(LogSourceControl, Warning, "Failed to unload some assets.");
		}

		TimeTakenUnload = FPlatformTime::Seconds() - TimeStartedUnload;
	}


	// Re-cache the SCC state...
	ISourceControlModule::Get().GetProvider().Execute(ISourceControlOperation::Create<FUpdateStatus>(), PackageFilenames, EConcurrency::Asynchronous);

	// Log timings...
	const double TimeTaken = FPlatformTime::Seconds() - TimeStarted;
	if (TimeTaken > 1.f)
	{
		TStringBuilder<512> TimingMessage;
		TimingMessage.Appendf(TEXT("ApplyOperationAndReloadPackages on %d file(s) took %.3fs"), InFilenames.Num(), TimeTaken);
		if (TimeTakenOperation > 0)
		{
			TimingMessage.Appendf(TEXT(" - operation: %.3fs"), TimeTakenOperation);
		}
		if (TimeTakenReload > 0)
		{
			TimingMessage.Appendf(TEXT(" - reload: %.3fs"), TimeTakenReload);
		}
		if (TimeTakenFlush > 0)
		{
			TimingMessage.Appendf(TEXT(" - flush: %.3fs"), TimeTakenFlush);
		}
		if (TimeTakenUnload > 0)
		{
			TimingMessage.Appendf(TEXT(" - unload: %.3fs"), TimeTakenUnload);
		}
		if (bReloadWorld)
		{
			TimingMessage.Appendf(TEXT(" - ReloadWorld"));
		}
		UE_LOGF(LogSourceControl, Log, "%ls", *TimingMessage);

		if (FEngineAnalytics::IsAvailable())
		{
			TArray<FAnalyticsEventAttribute> Attributes;
			Attributes.Emplace(TEXT("Duration.Seconds"), TimeTaken);
			Attributes.Emplace(TEXT("Duration.Seconds.Operation"), TimeTakenOperation);
			Attributes.Emplace(TEXT("Duration.Seconds.Reload"), TimeTakenReload);
			Attributes.Emplace(TEXT("Duration.Seconds.Flush"), TimeTakenFlush);
			Attributes.Emplace(TEXT("Duration.Seconds.Unload"), TimeTakenUnload);

			FEngineAnalytics::GetProvider().RecordEvent(TEXT("Editor.Usage.SourceControl.ApplyOperationAndReloadPackages"), Attributes);
		}
	}

	return bSuccess;
}

TArray<FString> USourceControlHelpers::GetSourceControlLocations(const bool bContentOnly)
{
	TArray<FString> SourceControlLocations;

	TArray<FSourceControlProjectInfo> CustomProjects = ISourceControlModule::Get().GetCustomProjects();
	if (CustomProjects.IsEmpty())
	{
		TArray<FString> RootPaths;
		FPackageName::QueryRootContentPaths(RootPaths);
		for (const FString& RootPath : RootPaths)
		{
			const FString RootPathOnDisk = FPackageName::LongPackageNameToFilename(RootPath);
			SourceControlLocations.Add(FPaths::ConvertRelativePathToFull(RootPathOnDisk));
		}

		if (!bContentOnly)
		{
			SourceControlLocations.Add(FPaths::ConvertRelativePathToFull(FPaths::ProjectConfigDir()));
			SourceControlLocations.Add(FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath()));
		}
	}
	else
	{
		for (FSourceControlProjectInfo& ProjectInfo : CustomProjects)
		{
			for (FString& ContentDir : ProjectInfo.ContentDirectories)
			{
				SourceControlLocations.Add(FPaths::ConvertRelativePathToFull(MoveTemp(ContentDir)));
			}
			if (!bContentOnly)
			{
				SourceControlLocations.Add(FPaths::ConvertRelativePathToFull(MoveTemp(ProjectInfo.ProjectDirectory)));
			}
		}
	}

	return SourceControlLocations;
}

bool USourceControlHelpers::ListRevertablePackages(TArray<FString>& OutRevertablePackageNames)
{
	if (!ISourceControlModule::Get().IsEnabled() || !ISourceControlModule::Get().GetProvider().IsAvailable())
	{
		return false;
	}

	// update status for all packages
	TArray<FString> Filenames = GetSourceControlLocations();
	
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	FSourceControlOperationRef Operation = ISourceControlOperation::Create<FUpdateStatus>();
	if (SourceControlProvider.Execute(Operation, Filenames) != ECommandResult::Succeeded)
	{
		return false;
	}

	// Get a list of all the revertable packages
	TMap<FString, FSourceControlStatePtr> PackageStates;
	FEditorFileUtils::FindAllSubmittablePackageFiles(PackageStates, true);

	TArray<FString> PackageNames;
	PackageStates.GetKeys(PackageNames);
	TMap<FString, FString> FileNamesToPackageNames;

	// Get a list of files pending delete
	TArray<FSourceControlStateRef> PendingDeleteItems = SourceControlProvider.GetCachedStateByPredicate(
		[&PackageNames, &FileNamesToPackageNames](const FSourceControlStateRef& State)
		{
			const FString& Filename = State->GetFilename();

			if (FPackageName::IsPackageFilename(Filename))
			{
				FString PackageName;
				FString FailureReason;
				if (!FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName, &FailureReason))
				{
					UE_LOGF(LogSourceControl, Warning, "%ls", *FailureReason);
					return false;
				}

				if (State->IsDeleted() && !PackageNames.Contains(PackageName))
				{
					FileNamesToPackageNames.Add(Filename, PackageName);
					return true;
				}
			}

			return false;
		}
	);

	// And append them to the list
	for (FSourceControlStateRef& Item : PendingDeleteItems)
	{
		const FString* PackageName = FileNamesToPackageNames.Find(Item->GetFilename());
		if (PackageName)
		{
			PackageStates.Add(*PackageName, Item);
		}
	}

	for (auto& PackageState : PackageStates)
	{
		const FString PackageName = PackageState.Key;
		OutRevertablePackageNames.Add(PackageName);
	}

	return true;
}

bool USourceControlHelpers::RevertAllChangesAndReloadWorld()
{
	TArray<FString> PackagesToReload;
	ListRevertablePackages(PackagesToReload);

	return RevertAndReloadPackages(PackagesToReload, /*bRevertAll=*/true, /*bReloadWorld=*/true);
}

bool USourceControlHelpers::RevertAndReloadPackages(const TArray<FString>& InPackagesToRevert, bool bRevertAll, bool bReloadWorld)
{
	auto RevertOperation = [bRevertAll](const TArray<FString>& InPackagesToRevert) -> bool
	{
		auto OperationCompleteCallback = FSourceControlOperationComplete::CreateLambda([InPackagesToRevert](const FSourceControlOperationRef& Operation, ECommandResult::Type InResult)
		{
			if (Operation->GetName() == TEXT("Revert"))
			{
				TSharedRef<FRevert> RevertOperation = StaticCastSharedRef<FRevert>(Operation);
				ISourceControlModule::Get().GetOnFilesDeleted().Broadcast(RevertOperation->GetDeletedFiles());

				// The revert may have caused added files to return to an uncontrolled state when not deleting newly added files
				// Handle that here, and return any newly uncontrolled files to the default uncontrolled changelist so they're not left in limbo
				if (GEditor && !RevertOperation->ShouldDeleteNewFiles())
				{
					TArray<FSourceControlStateRef> SCStates;
					if (ISourceControlModule::Get().GetProvider().GetState(InPackagesToRevert, SCStates, EStateCacheUsage::ForceUpdate) == ECommandResult::Succeeded)
					{
						TArray<FString> NewlyUncontrolledFiles;
						for (const FSourceControlStateRef& SCState : SCStates)
						{
							if (!SCState->IsSourceControlled())
							{
								NewlyUncontrolledFiles.Add(SCState->GetFilename());
							}
						}

						if (NewlyUncontrolledFiles.Num() > 0)
						{
							FScopedDisableSourceControl DisableSourceControl;
							GEditor->AddDeferredMarkForAddFiles(NewlyUncontrolledFiles);
							GEditor->RunDeferredMarkForAddFiles();
						}
					}
				}
			}
		});

		auto RevertOperation = ISourceControlOperation::Create<FRevert>();

		if (bRevertAll)
		{
			RevertOperation->SetRevertAll(true);
		}

		return ISourceControlModule::Get().GetProvider().Execute(RevertOperation, InPackagesToRevert, EConcurrency::Synchronous, OperationCompleteCallback) == ECommandResult::Succeeded;
	};

	return ApplyOperationAndReloadPackages(InPackagesToRevert,RevertOperation, bReloadWorld);
}
#endif //!WITH_EDITOR

bool USourceControlHelpers::RevertFiles(const TArray<FString>& InFiles,	bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	// Determine file type and ensure they are in form source control wants
	// Even if some files were skipped, still apply to the others
	TArray<FString> SCFiles;
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, SCFiles, bSilent);
	const int32 NumFiles = SCFiles.Num();

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);
	if (!Provider)
	{
		// Error or can't communicate with source control
		return false;
	}

	TArray<FSourceControlStateRef> SCStates;
	Provider->GetState(SCFiles, SCStates, EStateCacheUsage::ForceUpdate);

	TArray<FString> SCFilesToRevert;
	for (int32 Index = 0; Index < NumFiles; ++Index)
	{
		FString SCFile = SCFiles[Index];
		FSourceControlStateRef SCState = SCStates[Index];

		// Less error checking and info is made for multiple files than the single file version.
		// This multi-file version could be made similarly more sophisticated.
		if (SCState->CanRevert())
		{
			SCFilesToRevert.Add(SCFile);
		}
	}

	bool bSuccess = !bFilesSkipped;
	if (SCFilesToRevert.Num())
	{
		bSuccess &= Provider->Execute(ISourceControlOperation::Create<FRevert>(), SCFilesToRevert) == ECommandResult::Succeeded;
	}

	return bSuccess;
}


bool USourceControlHelpers::RevertUnchangedFile(const FString& InFile, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	// Only revert file if they haven't had any changes made

	// Stuff single file in array for functions that require array
	TArray<FString> InFiles;
	InFiles.Add(SCFile);

	RevertUnchangedFiles(*Provider, InFiles);

	// Assume it succeeded
	return true;
}


bool USourceControlHelpers::RevertUnchangedFiles(const TArray<FString>& InFiles, bool bSilent)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	// Determine file types and ensure they are in form source control wants
	TArray<FString> FilePaths;
	SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, FilePaths, bSilent);

	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	// Less error checking and info is made for multiple files than the single file version.
	// This multi-file version could be made similarly more sophisticated.

	// Only revert files if they haven't had any changes made
	RevertUnchangedFiles(*Provider, FilePaths);

	// Assume it succeeded
	return true;
}


bool USourceControlHelpers::CheckInFile(const FString& InFile, const FString& InDescription, bool bSilent, bool bKeepCheckedOut)
{
	// Determine file type and ensure it is in form source control wants
	FString SCFile = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InFile, bSilent);

	if (SCFile.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOp = ISourceControlOperation::Create<FCheckIn>();
	CheckInOp->SetDescription(FText::FromString(InDescription));

	CheckInOp->SetKeepCheckedOut(bKeepCheckedOut);

	ECommandResult::Type Result = Provider->Execute(CheckInOp, SCFile);

	return Result == ECommandResult::Succeeded;
}


bool USourceControlHelpers::CheckInFiles(const TArray<FString>& InFiles, const FString& InDescription, bool bSilent, bool bKeepCheckedOut)
{
	// If we have nothing to process, exit immediately
	if (InFiles.IsEmpty())
	{
		return true;
	}

	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	TArray<FString> FilePaths;

	// Even if some files were skipped, still apply to the others
	bool bFilesSkipped = !SourceControlHelpersInternal::ConvertFilesToQualifiedPaths(InFiles, FilePaths, bSilent);

	// Less error checking and info is made for multiple files than the single file version.
	// This multi-file version could be made similarly more sophisticated.
	TSharedRef<FCheckIn, ESPMode::ThreadSafe> CheckInOp = ISourceControlOperation::Create<FCheckIn>();
	CheckInOp->SetDescription(FText::FromString(InDescription));

	CheckInOp->SetKeepCheckedOut(bKeepCheckedOut);

	ECommandResult::Type Result = Provider->Execute(CheckInOp, FilePaths);

	return !bFilesSkipped && (Result == ECommandResult::Succeeded);
}


bool USourceControlHelpers::CopyFile(const FString& InSourcePath, const FString& InDestPath, bool bSilent)
{
	// Determine file type and ensure it is in form source control wants
	FString SCSource = SourceControlHelpersInternal::ConvertFileToQualifiedPath(InSourcePath, bSilent);

	if (SCSource.IsEmpty())
	{
		return false;
	}

	// Determine file type and ensure it is in form source control wants
	FString SCSourcExt(FPaths::GetExtension(SCSource, true));
	FString SCDest(SourceControlHelpersInternal::ConvertFileToQualifiedPath(InDestPath, bSilent, /*bAllowDirectories*/false, *SCSourcExt));

	if (SCDest.IsEmpty())
	{
		return false;
	}

	// Ensure source control system is up and running
	ISourceControlProvider* Provider = SourceControlHelpersInternal::VerifySourceControl(bSilent);

	if (!Provider)
	{
		return false;
	}

	TSharedRef<FCopy, ESPMode::ThreadSafe> CopyOp = ISourceControlOperation::Create<FCopy>();
	CopyOp->SetDestination(SCDest);

	ECommandResult::Type Result = Provider->Execute(CopyOp, SCSource);

	return Result == ECommandResult::Succeeded;
}

TArray<FSourceControlState> USourceControlHelpers::QueryFileStates(const TArray<FString> InFiles, bool bSilent, bool bUseSourceControlStateCache)
{
	TArray<FSourceControlState> States;

	SourceControlHelpersInternal::RunQueryFileStates(InFiles, EConcurrency::Synchronous, bSilent, bUseSourceControlStateCache,
		[&States](FSourceControlState FileStateOut)
		{
			States.Emplace(MoveTemp(FileStateOut));
		});

	return States;
}

FSourceControlState USourceControlHelpers::QueryFileState(const FString& InFile, bool bSilent, bool bUseSourceControlStateCache)
{
	FSourceControlState State;
	State.Filename = InFile;
	SourceControlHelpersInternal::RunQueryFileStates(TArray<FString> {InFile}, EConcurrency::Synchronous, bSilent, bUseSourceControlStateCache,
		[&State](FSourceControlState FileStateOut)
		{
			State = MoveTemp(FileStateOut);
		});

	return State;
}

void USourceControlHelpers::AsyncQueryFileStates(FQueryFileStateDelegate FileStateCallback, TArray<FString> InFiles, bool bSilent)
{
	bool constexpr bUseCachedSourceControlState = false;
	SourceControlHelpersInternal::RunQueryFileStates(InFiles, EConcurrency::Asynchronous, bSilent, bUseCachedSourceControlState,
		[FileStateCallback](FSourceControlState InState)
		{
			FileStateCallback.ExecuteIfBound(InState);
		});
}

void USourceControlHelpers::AsyncQueryFileState(FQueryFileStateDelegate FileStateCallback, const FString& InFile, bool bSilent)
{
	AsyncQueryFileStates(FileStateCallback, TArray<FString> {InFile}, bSilent);
}

bool USourceControlHelpers::GetFilesInDepotAtPath(const FString& Path, TArray<FString>& OutFilesList, bool bIncludeDeleted, bool bSilent, bool bIsFileRegexSearch)
{
	TArray<FString> Paths;
	Paths.Add(Path);
	return GetFilesInDepotAtPaths(Paths, OutFilesList, bIncludeDeleted, bSilent, bIsFileRegexSearch);
}

bool USourceControlHelpers::GetFilesInDepotAtPaths(const TArray<FString>& Paths, TArray<FString>& OutFilesList, bool bIncludeDeleted, bool bSilent, bool bIsFileRegexSearch)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USourceControlHelpers::GetFilesInDepotAtPaths);

	bool bSuccess = false;
	if (ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		if (SourceControlProvider.IsAvailable())
		{
			TArray<FString> PathsToQuery;
			for (const FString& Path : Paths)
			{
				FString RelativePath;
				FPackageName::TryConvertLongPackageNameToFilename(Path, RelativePath);
				FPaths::RemoveDuplicateSlashes(RelativePath);
				PathsToQuery.Add(RelativePath);
			}

			TSharedRef<FGetFileList, ESPMode::ThreadSafe> Operation = ISourceControlOperation::Create<FGetFileList>();
			Operation->SetIncludeDeleted(bIncludeDeleted);
			Operation->SetQuiet(bSilent);
			Operation->SetSearchPattern(PathsToQuery);
			if (bIsFileRegexSearch)
			{
				Operation->SetMethodUsed(FGetFileList::EGetFileListMethod::FileRegexSearch);
			}

			ECommandResult::Type Result = SourceControlProvider.Execute(Operation, PathsToQuery, EConcurrency::Synchronous);
			bSuccess = (Result == ECommandResult::Succeeded);

			if (!bSuccess)
			{
				for (const FString& Path : Paths)
				{
					FFormatNamedArguments Arguments;
					Arguments.Add(TEXT("Path"), FText::FromString(Path));
					SourceControlHelpersInternal::LogError(FText::Format(LOCTEXT("CouldNotGetFileList", "Could not get file list at path: {Path}."), Arguments), bSilent);
				}
			}
			else
			{
				OutFilesList = Operation->GetFilesList();
			}
		}
	}
	return bSuccess;
}

static FString PackageFilename_Internal( const FString& InPackageName )
{
	FString Filename = InPackageName;

	// Get the filename by finding it on disk first
	if ( !FPackageName::IsMemoryPackage(InPackageName) && !FPackageName::DoesPackageExist(InPackageName, &Filename) )
	{
		// The package does not exist on disk, see if we can find it in memory and predict the file extension
		// Only do this if the supplied package name is valid
		const bool bIncludeReadOnlyRoots = false;
		if ( FPackageName::IsValidLongPackageName(InPackageName, bIncludeReadOnlyRoots) )
		{
			UPackage* Package = FindPackage(nullptr, *InPackageName);
			// This is a package in memory that has not yet been saved. Determine the extension and convert to a filename, if we do have the package, just assume normal asset extension
			const FString PackageExtension = Package && Package->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			Filename = FPackageName::LongPackageNameToFilename(InPackageName, PackageExtension);
		}
	}

	return Filename;
}


FString USourceControlHelpers::PackageFilename( const FString& InPackageName )
{
	return FPaths::ConvertRelativePathToFull(PackageFilename_Internal(InPackageName));
}


FString USourceControlHelpers::PackageFilename( const UPackage* InPackage )
{
	FString Filename;
	if(InPackage != nullptr)
	{
		// Prefer using package loaded path to resolve file name as it properly resolves memory packages
		FString PackageLoadedPath = InPackage->GetLoadedPath().GetPackageName();
		if (!InPackage->GetLoadedPath().IsEmpty() && FPackageName::IsMemoryPackage(PackageLoadedPath))
		{
			const FString PackageExtension = InPackage->ContainsMap() ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();
			Filename = FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(PackageLoadedPath, PackageExtension));
		}
		else
		{
			Filename = FPaths::ConvertRelativePathToFull(PackageFilename_Internal(InPackage->GetName()));
		}
	}
	return Filename;
}


TArray<FString> USourceControlHelpers::PackageFilenames( const TArray<UPackage*>& InPackages )
{
	TArray<FString> OutNames;
	OutNames.Reserve(InPackages.Num());

	for (int32 PackageIndex = 0; PackageIndex < InPackages.Num(); PackageIndex++)
	{
		OutNames.Add(FPaths::ConvertRelativePathToFull(PackageFilename(InPackages[PackageIndex])));
	}

	return OutNames;
}


TArray<FString> USourceControlHelpers::PackageFilenames( const TArray<FString>& InPackageNames )
{
	TArray<FString> OutNames;
	OutNames.Reserve(InPackageNames.Num());

	for (int32 PackageIndex = 0; PackageIndex < InPackageNames.Num(); PackageIndex++)
	{
		OutNames.Add(FPaths::ConvertRelativePathToFull(PackageFilename_Internal(InPackageNames[PackageIndex])));
	}

	return OutNames;
}


TArray<FString> USourceControlHelpers::AbsoluteFilenames( const TArray<FString>& InFileNames )
{
	TArray<FString> AbsoluteFiles;
	AbsoluteFiles.Reserve(InFileNames.Num());

	for (const FString& FileName : InFileNames)
	{
		if(!FPaths::IsRelative(FileName))
		{
			AbsoluteFiles.Add(FileName);
		}
		else
		{
			AbsoluteFiles.Add(FPaths::ConvertRelativePathToFull(FileName));
		}

		FPaths::NormalizeFilename(AbsoluteFiles[AbsoluteFiles.Num() - 1]);
	}

	return AbsoluteFiles;
}


void USourceControlHelpers::RevertUnchangedFiles( ISourceControlProvider& InProvider, const TArray<FString>& InFiles )
{
	// Make sure we update the modified state of the files
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateModifiedState(true);
	InProvider.Execute(UpdateStatusOperation, InFiles);

	TArray<FString> UnchangedFiles;
	TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> > OutStates;
	InProvider.GetState(InFiles, OutStates, EStateCacheUsage::Use);

	for(TArray< TSharedRef<ISourceControlState, ESPMode::ThreadSafe> >::TConstIterator It(OutStates); It; It++)
	{
		TSharedRef<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = *It;
		if(SourceControlState->IsCheckedOut() && !SourceControlState->IsModified())
		{
			UnchangedFiles.Add(SourceControlState->GetFilename());
		}
	}

	if(UnchangedFiles.Num())
	{
		InProvider.Execute( ISourceControlOperation::Create<FRevert>(), UnchangedFiles );
	}
}


bool USourceControlHelpers::AnnotateFile( ISourceControlProvider& InProvider, const FString& InLabel, const FString& InFile, TArray<FAnnotationLine>& OutLines )
{
	TArray< TSharedRef<ISourceControlLabel> > Labels = InProvider.GetLabels( InLabel );
	if(Labels.Num() > 0)
	{
		TSharedRef<ISourceControlLabel> Label = Labels[0];
		TArray< TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> > Revisions;
		Label->GetFileRevisions(InFile, Revisions);
		if(Revisions.Num() > 0)
		{
			TSharedRef<ISourceControlRevision, ESPMode::ThreadSafe> Revision = Revisions[0];
			if(Revision->GetAnnotated(OutLines))
			{
				return true;
			}
		}
	}

	return false;
}

bool USourceControlHelpers::AnnotateFile( ISourceControlProvider& InProvider, int32 InCheckInIdentifier, const FString& InFile, TArray<FAnnotationLine>& OutLines )
{
	TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
	UpdateStatusOperation->SetUpdateHistory(true);
	if(InProvider.Execute(UpdateStatusOperation, InFile) == ECommandResult::Succeeded)
	{
		FSourceControlStatePtr State = InProvider.GetState(InFile, EStateCacheUsage::Use);
		if(State.IsValid())
		{
			for(int32 HistoryIndex = State->GetHistorySize() - 1; HistoryIndex >= 0; HistoryIndex--)
			{
				// check that the changelist corresponds to this revision - we assume history is in latest-first order
				TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = State->GetHistoryItem(HistoryIndex);
				if(Revision.IsValid() && Revision->GetCheckInIdentifier() >= InCheckInIdentifier)
				{
					if(Revision->GetAnnotated(OutLines))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}


bool USourceControlHelpers::CheckoutOrMarkForAdd( const FString& InDestFile, const FText& InFileDescription, const FOnPostCheckOut& OnPostCheckOut, FText& OutFailReason )
{
	bool bSucceeded = true;

	ISourceControlProvider& Provider = ISourceControlModule::Get().GetProvider();

	// first check for source control check out
	if (ISourceControlModule::Get().IsEnabled())
	{
		FSourceControlStatePtr SourceControlState = Provider.GetState(InDestFile, EStateCacheUsage::ForceUpdate);
		if (SourceControlState.IsValid())
		{
			if (SourceControlState->IsSourceControlled() && SourceControlState->CanCheckout())
				{
					ECommandResult::Type Result = Provider.Execute(ISourceControlOperation::Create<FCheckOut>(), InDestFile);
					bSucceeded = (Result == ECommandResult::Succeeded);
					if (!bSucceeded)
					{
						OutFailReason = FText::Format(LOCTEXT("SourceControlCheckoutError", "Could not check out {0} file."), InFileDescription);
					}
				}
		}
	}

	if (bSucceeded)
	{
		if(OnPostCheckOut.IsBound())
		{
			bSucceeded = OnPostCheckOut.Execute(InDestFile, InFileDescription, OutFailReason);
		}
	}

	// mark for add now if needed
	if (bSucceeded && ISourceControlModule::Get().IsEnabled())
	{
		FSourceControlStatePtr SourceControlState = Provider.GetState(InDestFile, EStateCacheUsage::Use);
		if (SourceControlState.IsValid())
		{
			if (!SourceControlState->IsSourceControlled())
			{
				ECommandResult::Type Result = Provider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), InDestFile);
				bSucceeded = (Result == ECommandResult::Succeeded);
				if (!bSucceeded)
				{
					OutFailReason = FText::Format(LOCTEXT("SourceControlMarkForAddError", "Could not mark {0} file for add."), InFileDescription);
				}
			}
		}
	}

	return bSucceeded;
}


bool USourceControlHelpers::CopyFileUnderSourceControl( const FString& InDestFile, const FString& InSourceFile, const FText& InFileDescription, FText& OutFailReason)
{
	struct Local
	{
		static bool CopyFile(const FString& InDestinationFile, const FText& InFileDesc, FText& OutFailureReason, FString InFileToCopy)
		{
			const bool bReplace = true;
			const bool bEvenIfReadOnly = true;
			bool bSucceeded = (IFileManager::Get().Copy(*InDestinationFile, *InFileToCopy, bReplace, bEvenIfReadOnly) == COPY_OK);
			if (!bSucceeded)
			{
				OutFailureReason = FText::Format(LOCTEXT("ExternalImageCopyError", "Could not overwrite {0} file."), InFileDesc);
			}

			return bSucceeded;
		}
	};

	return CheckoutOrMarkForAdd(InDestFile, InFileDescription, FOnPostCheckOut::CreateStatic(&Local::CopyFile, InSourceFile), OutFailReason);
}

namespace
{
	bool CopyPackage_Internal(UPackage* DestPackage, UPackage* SourcePackage, FCopy::ECopyMethod CopyMethod, EStateCacheUsage::Type StateCacheUsage)
	{
		if (ISourceControlModule::Get().IsEnabled())
		{
			ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

			const FString SourceFilename = USourceControlHelpers::PackageFilename(SourcePackage);
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceFilename, StateCacheUsage);
			if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
			{
				FString DestinationPath = USourceControlHelpers::PackageFilename(DestPackage);
				FSourceControlStatePtr DestinationState = SourceControlProvider.GetState(DestinationPath, StateCacheUsage);
				TSharedRef<FCopy, ESPMode::ThreadSafe> CopyOperation = ISourceControlOperation::Create<FCopy>();
				CopyOperation->SetDestination(DestinationPath);
				CopyOperation->CopyMethod = CopyMethod;
				CopyOperation->ResolveMethod = (DestinationState.IsValid() && !DestinationState->CanRevert() && !IFileManager::Get().FileExists(*DestinationPath)) ? FCopy::EResolveMethod::AcceptSource : FCopy::EResolveMethod::AcceptTarget;

				return (SourceControlProvider.Execute(CopyOperation, SourceFilename) == ECommandResult::Succeeded);
			}
		}

		return false;
	}
}

bool USourceControlHelpers::BranchPackage(UPackage* DestPackage, UPackage* SourcePackage, EStateCacheUsage::Type StateCacheUsage)
{
	return CopyPackage_Internal(DestPackage, SourcePackage, FCopy::ECopyMethod::Branch, StateCacheUsage);
}

bool USourceControlHelpers::CopyPackage(UPackage* DestPackage, UPackage* SourcePackage, EStateCacheUsage::Type StateCacheUsage)
{
	return CopyPackage_Internal(DestPackage, SourcePackage, FCopy::ECopyMethod::Add, StateCacheUsage);
}


const FString& USourceControlHelpers::GetSettingsIni()
{
	if (ISourceControlModule::Get().GetUseGlobalSettings())
	{
		return GetGlobalSettingsIni();
	}
	else
	{
		static FString SourceControlSettingsIni;
		if (SourceControlSettingsIni.Len() == 0)
		{
			FConfigContext Context = FConfigContext::ReadIntoGConfig();
			Context.Load(TEXT("SourceControlSettings"), SourceControlSettingsIni);
		}
		return SourceControlSettingsIni;
	}
}


const FString& USourceControlHelpers::GetGlobalSettingsIni()
{
	static FString SourceControlGlobalSettingsIni;
	if (SourceControlGlobalSettingsIni.Len() == 0)
	{
		FConfigContext Context = FConfigContext::ReadIntoGConfig();
		Context.GeneratedConfigDir = FPaths::EngineSavedDir() + TEXT("Config/");
		Context.ProjectConfigDir = (""); // don't load anything from project configs
		Context.Load(TEXT("SourceControlSettings"), SourceControlGlobalSettingsIni);
	}
	return SourceControlGlobalSettingsIni;
}

bool USourceControlHelpers::GetAssetData(const FString& InFileName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies)
{
	FString PackageName;
	if (FPackageName::TryConvertFilenameToLongPackageName(InFileName, PackageName))
	{
		return GetAssetData(InFileName, PackageName, OutAssets, OutDependencies);
	}
	else
	{
		return false;
	}
}

bool USourceControlHelpers::GetAssetDataFromPackage(const FString& PackageName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies)
{
	return GetAssetData(PackageFilename(PackageName), PackageName, OutAssets, OutDependencies);
}

bool USourceControlHelpers::GetAssetData(const FString & InFileName, const FString& InPackageName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies)
{
	const bool bGetDependencies = (OutDependencies != nullptr);
	OutAssets.Reset();
	if (bGetDependencies)
	{
		OutDependencies->Reset();
	}

	// Try the registry first
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().GetAssetsByPackageName(*InPackageName, OutAssets, true);

	if (OutAssets.Num() > 0)
	{
		// Assets are already in the cache, we can query dependencies directly
		if (bGetDependencies)
		{
			AssetRegistryModule.Get().GetDependencies(*InPackageName, *OutDependencies, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
		}

		return true;
	}

	// Filter on improbable file extensions
	EPackageExtension PackageExtension = FPackagePath::ParseExtension(InFileName);

	if (PackageExtension == EPackageExtension::Unspecified ||
		PackageExtension == EPackageExtension::Custom)
	{
		return false;
	}

	// If nothing was done, try to get the data explicitly	
	IAssetRegistry::FLoadPackageRegistryData LoadedData(bGetDependencies);

	AssetRegistryModule.Get().LoadPackageRegistryData(InFileName, LoadedData);
	OutAssets = MoveTemp(LoadedData.Data);

	if (bGetDependencies)
	{
		*OutDependencies = MoveTemp(LoadedData.DataDependencies);
	}	

	return OutAssets.Num() > 0;
}

bool USourceControlHelpers::GetAssetDataFromFileHistory(const FString& InFileName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies, int64 MaxFetchSize)
{
	OutAssets.Reset();

	if (OutDependencies)
	{
		OutDependencies->Reset();
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	// Get the SCC state
	FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(InFileName, EStateCacheUsage::Use);
	if (SourceControlState.IsValid())
	{
		return GetAssetDataFromFileHistory(SourceControlState, OutAssets, OutDependencies, MaxFetchSize);
	}
	else
	{
		return false;
	}
}

bool USourceControlHelpers::GetAssetDataFromFileHistory(FSourceControlStatePtr InSourceControlState, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies /* = nullptr */, int64 MaxFetchSize /* = -1 */)
{
	check(InSourceControlState.IsValid());
	OutAssets.Reset();

	if (OutDependencies)
	{
		OutDependencies->Reset();
	}

	// This code is similar to what's done in UAssetToolsImpl::DiffAgainstDepot but we'll force it quiet to prevent recursion issues
	if (InSourceControlState->GetHistorySize() == 0)
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateHistory(true);
		UpdateStatusOperation->SetQuiet(true);
		SourceControlProvider.Execute(UpdateStatusOperation, InSourceControlState->GetFilename());
	}

	if (InSourceControlState->GetHistorySize() > 0)
	{
		TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> Revision = InSourceControlState->GetHistoryItem(0);
		check(Revision.IsValid());

		const bool bShouldGetFile = (MaxFetchSize < 0 || MaxFetchSize >(int64)Revision->GetFileSize());

		FString TempFileName;
		if (bShouldGetFile && Revision->Get(TempFileName))
		{
			return GetAssetData(TempFileName, OutAssets, OutDependencies);
		}
	}

	return false;
}

FScopedSourceControl::FScopedSourceControl()
{
	bInitSourceControl = !ISourceControlModule::Get().GetProvider().IsAvailable();
	if (bInitSourceControl)
	{
		ISourceControlModule::Get().GetProvider().Init();
	}
}


FScopedSourceControl::~FScopedSourceControl()
{
	if (bInitSourceControl)
	{
		ISourceControlModule::Get().GetProvider().Close();
	}
}


ISourceControlProvider& FScopedSourceControl::GetProvider()
{
	return ISourceControlModule::Get().GetProvider();
}


#undef LOCTEXT_NAMESPACE

