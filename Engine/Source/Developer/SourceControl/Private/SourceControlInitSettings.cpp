// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlInitSettings.h"

#include "Containers/StringView.h"

FSourceControlInitSettings::FSourceControlInitSettings(EBehavior InBehavior)
	: OverrideBehavior(InBehavior)
{

}

FSourceControlInitSettings::FSourceControlInitSettings(EBehavior InBehavior, ECmdLineFlags InCmdLineFlags)
	: OverrideBehavior(InBehavior)
	, CmdLineFlags(InCmdLineFlags)
{

}

void FSourceControlInitSettings::SetConfigBehavior(EConfigBehavior InBehavior)
{
	ConfigBehavior = InBehavior;
}

bool FSourceControlInitSettings::CanWriteToConfigFile() const
{
	return ConfigBehavior == EConfigBehavior::ReadWrite;
}

bool FSourceControlInitSettings::CanReadFromConfigFile() const
{
	return ConfigBehavior != EConfigBehavior::None;
}

void FSourceControlInitSettings::AddSetting(FStringView SettingName, FStringView SettingValue)
{
	Settings.Add(FString(SettingName), FString(SettingValue));
}

void FSourceControlInitSettings::OverrideSetting(FStringView SettingName, FString& InOutSettingValue)
{
	const int32 Hash = GetTypeHash(SettingName);
	FString* InitialValue = Settings.FindByHash(Hash, SettingName);

	if (InitialValue != nullptr)
	{
		InOutSettingValue = *InitialValue;
	}
	else if (OverrideBehavior == EBehavior::OverrideAll)
	{
		InOutSettingValue.Empty();
	}
}

bool FSourceControlInitSettings::HasOverrides() const
{
	return !Settings.IsEmpty();
}

bool FSourceControlInitSettings::IsOverridden(FStringView SettingName) const
{
	const int32 Hash = GetTypeHash(SettingName);
	return Settings.FindByHash(Hash, SettingName) != nullptr;
}

void FSourceControlInitSettings::SetCmdLineFlags(ECmdLineFlags Flags)
{
	CmdLineFlags = Flags;
}

bool FSourceControlInitSettings::ShouldReadFromCmdLine() const
{
	return CmdLineFlags == ECmdLineFlags::ReadAll;
}
