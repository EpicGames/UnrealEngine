// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlModule.h"
#include "Features/IModularFeatures.h"
#include "SSourceControlLogin.h"
#include "SourceControlOperations.h"
#include "SourceControlHelpers.h"
#include "SourceControlAssetDataCache.h"
#include "SourceControlFileStatusMonitor.h"
#include "SourceControlCVars.h"
#include "Misc/Paths.h"

#if SOURCE_CONTROL_WITH_SLATE
	#include "Widgets/DeclarativeSyntaxSupport.h"
	#include "Widgets/SWindow.h"
	#include "Widgets/Layout/SBox.h"
	#include "Framework/Docking/TabManager.h"
	#include "Framework/Application/SlateApplication.h"
#endif

#if WITH_UNREAL_DEVELOPER_TOOLS
	#include "MessageLogModule.h"
#endif

#if WITH_EDITOR
	#include "Editor.h"
	#include "EngineAnalytics.h"
	#include "Interfaces/IAnalyticsProvider.h"
#endif

DEFINE_LOG_CATEGORY(LogSourceControl);

#define LOCTEXT_NAMESPACE "SourceControl"

static const FName SourceControlFeatureName("SourceControl");

namespace SourceControlConstants
{
	/** The interval at which we refresh a file's state */
	const FTimespan StateRefreshInterval = FTimespan::FromMinutes(5.0);
}

FScopedDisableSourceControl::FScopedDisableSourceControl(const bool InDisable)
	: bDisabled(InDisable)
{
	checkf(IsInGameThread(), TEXT("FScopedDisableSourceControl should only be used from the game thread!"));
	if (bDisabled)
	{
		++FSourceControlModule::Get().TemporarilyDisabledCount;
	}
}

FScopedDisableSourceControl::~FScopedDisableSourceControl()
{
	checkf(IsInGameThread(), TEXT("FScopedDisableSourceControl should only be used from the game thread!"));
	if (bDisabled)
	{
		--FSourceControlModule::Get().TemporarilyDisabledCount;
	}
}

FSourceControlModule::FSourceControlModule()
	: CurrentSourceControlProvider(NULL)
	, TemporarilyDisabledCount(0)
{
}

void FSourceControlModule::StartupModule()
{
	// load our settings
	SourceControlSettings.LoadSettings();

	// Register to check for source control features
	IModularFeatures::Get().OnModularFeatureRegistered().AddRaw(this, &FSourceControlModule::HandleModularFeatureRegistered);
	IModularFeatures::Get().OnModularFeatureUnregistered().AddRaw(this, &FSourceControlModule::HandleModularFeatureUnregistered);

	// bind default provider to editor
	IModularFeatures::Get().RegisterModularFeature( SourceControlFeatureName, &DefaultSourceControlProvider );

#if WITH_UNREAL_DEVELOPER_TOOLS
	// create a message log for source control to use
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	MessageLogModule.RegisterLogListing("SourceControl", LOCTEXT("SourceControlLogLabel", "Revision Control"));
#endif

	AssetDataCache.Startup();

#if WITH_EDITOR
	FEditorDelegates::OnAssetsCanDelete.AddRaw(this, &FSourceControlModule::OnAssetsCanDelete);
#endif
	
	SourceControlFileStatusMonitor = MakeShared<FSourceControlFileStatusMonitor>();
}

void FSourceControlModule::ShutdownModule()
{
	AssetDataCache.Shutdown();
	// close the current provider
	GetProvider().Close();

#if WITH_UNREAL_DEVELOPER_TOOLS
	// unregister message log
	if(FModuleManager::Get().IsModuleLoaded("MessageLog"))
	{
		FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
		MessageLogModule.UnregisterLogListing("SourceControl");
	}
#endif

	// unbind default provider from editor
	IModularFeatures::Get().UnregisterModularFeature( SourceControlFeatureName, &DefaultSourceControlProvider );

	// we don't care about modular features any more
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);

#if WITH_EDITOR
	FEditorDelegates::OnAssetsCanDelete.RemoveAll(this);
#endif

	SourceControlFileStatusMonitor.Reset();
}

void FSourceControlModule::SaveSettings()
{
	SourceControlSettings.SaveSettings();
}

void FSourceControlModule::ShowLoginDialog(const FSourceControlLoginClosed& InOnSourceControlLoginClosed, ELoginWindowMode::Type InLoginWindowMode, EOnLoginWindowStartup::Type InOnLoginWindowStartup)
{
#if SOURCE_CONTROL_WITH_SLATE
	// Avoid continue running tasks while the dialog is displayed. Any change in textboxes could affect the connection status.
	AssetDataCache.OnSourceControlDialogShown();

	// Get Active Provider Name
	ActiveProviderName = GetProvider().GetName().ToString();

	// if we are forcing a modal dialog, change the preference now
	ELoginWindowMode::Type LoginWindowMode = InLoginWindowMode;
	if(SourceControlCVars::CVarSourceControlEnableLoginDialogModal.GetValueOnAnyThread())
	{
		LoginWindowMode = ELoginWindowMode::Modal;
	}

	// if we are showing a modal version of the dialog & a modeless version already exists, we must destroy the modeless dialog first
	if(LoginWindowMode == ELoginWindowMode::Modal && SourceControlLoginPtr.IsValid())
	{
		// unhook the delegate so it doesn't fire in this case
		SourceControlLoginWindowPtr->SetOnWindowClosed(FOnWindowClosed());
		SourceControlLoginWindowPtr->RequestDestroyWindow();
		SourceControlLoginWindowPtr = NULL;
		SourceControlLoginPtr = NULL;
	}

	if(SourceControlLoginWindowPtr.IsValid())
	{
		SourceControlLoginWindowPtr->BringToFront();
	}
	else
	{
		// set provider to 'none'.
		// When we open the window, we turn off the fact that source control is available, this solves issues that are present with
		// being a three state modeless system (Accepted settings, disabled, and not yet decided).
		if(InOnLoginWindowStartup == EOnLoginWindowStartup::ResetProviderToNone)
		{
			SetProvider("None");
		}

		// temporarily disable access to source control features
		++TemporarilyDisabledCount;

		// Create the window
		SourceControlLoginWindowPtr = SNew(SWindow)
			.Title( LOCTEXT("SourceControlLoginTitle", "Revision Control Login") )
			.HasCloseButton(false)
			.SupportsMaximize(false) 
			.SupportsMinimize(false)
			.SizingRule( ESizingRule::Autosized );

		// Set the closed callback
		SourceControlLoginWindowPtr->SetOnWindowClosed(FOnWindowClosed::CreateRaw(this, &FSourceControlModule::OnSourceControlDialogClosed));

		// Setup the content for the created login window.
		SourceControlLoginWindowPtr->SetContent(
				SAssignNew(SourceControlLoginPtr, SSourceControlLogin)
				.ParentWindow(SourceControlLoginWindowPtr)
				.OnSourceControlLoginClosed(InOnSourceControlLoginClosed)
			);

		TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();
		if(RootWindow.IsValid())
		{
			if(LoginWindowMode == ELoginWindowMode::Modal)
			{
				FSlateApplication::Get().AddModalWindow(SourceControlLoginWindowPtr.ToSharedRef(), RootWindow);
			}
			else
			{
				FSlateApplication::Get().AddWindowAsNativeChild(SourceControlLoginWindowPtr.ToSharedRef(), RootWindow.ToSharedRef());
			}
		}
		else
		{
			if(LoginWindowMode == ELoginWindowMode::Modal)
			{
				FSlateApplication::Get().AddModalWindow(SourceControlLoginWindowPtr.ToSharedRef(), RootWindow);
			}
			else
			{
				FSlateApplication::Get().AddWindow(SourceControlLoginWindowPtr.ToSharedRef());
			}
		}
	}
#else
	STUBBED("FSourceControlModule::ShowLoginDialog - no Slate");
#endif // SOURCE_CONTROL_WITH_SLATE
}

void FSourceControlModule::OnSourceControlDialogClosed(const TSharedRef<SWindow>& InWindow)
{	
	SourceControlLoginWindowPtr = NULL;
	SourceControlLoginPtr = NULL;
	--TemporarilyDisabledCount;

#if WITH_EDITOR
	FString NewProvider = CurrentSourceControlProvider->GetName().ToString();
	if( FEngineAnalytics::IsAvailable() && !ActiveProviderName.Equals( NewProvider, ESearchCase::IgnoreCase ))
	{
		FEngineAnalytics::GetProvider().RecordEvent( TEXT( "Editor.Usage.SourceControl" ), TEXT( "Provider" ), NewProvider );
		ActiveProviderName = NewProvider;
	}
#endif

	AssetDataCache.OnSourceControlDialogClosed();
}

void FSourceControlModule::InitializeSourceControlProviders()
{
	int32 SourceControlCount = IModularFeatures::Get().GetModularFeatureImplementationCount(SourceControlFeatureName);
	if( SourceControlCount > 0 )
	{
		FString PreferredSourceControlProvider = SourceControlSettings.GetProvider();
		TArray<ISourceControlProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ISourceControlProvider>(SourceControlFeatureName);
		for(auto It(Providers.CreateIterator()); It; It++)
		{ 
			ISourceControlProvider* Provider = *It;
			if(PreferredSourceControlProvider == Provider->GetName().ToString())
			{
				CurrentSourceControlProvider = Provider;
				break;
			}
		}

		// no provider found of this name, default to the first one
		if( CurrentSourceControlProvider == NULL )
		{
			CurrentSourceControlProvider = &DefaultSourceControlProvider;
		}
	}

	check(CurrentSourceControlProvider);

	CurrentSourceControlProvider->Init(false);	// Don't force a connection here, as its synchronous. Let the user establish a connection.
}

void FSourceControlModule::GetProviderNames(TArray<FName>& OutProviderNames)
{
	OutProviderNames.Reset();

	int32 ProviderCount = GetNumSourceControlProviders();
	for ( int32 ProviderIndex = 0; ProviderIndex < ProviderCount; ProviderIndex++ )
	{
		FName ProviderName = GetSourceControlProviderName(ProviderIndex);
		OutProviderNames.Add(ProviderName);
	}
}

void FSourceControlModule::Tick()
{
	if (CurrentSourceControlProvider != nullptr)
	{
		ISourceControlProvider& Provider = GetProvider();

		// tick the provider, so any operation results can be read back
		Provider.Tick();

		AssetDataCache.Tick();

		// don't allow background status updates when temporarily disabled for login
		if (TemporarilyDisabledCount == 0)
		{
			// determine delay before processing pending requests
			const int32 Delay = SourceControlCVars::CVarSourceControlIntervalBatchStatusUpdate.GetValueOnGameThread();

			// check for any pending dispatches
			const FDateTime Now = FDateTime::UtcNow();
			if ((PendingStatusUpdateFiles.Num() > 0) && (PendingStatusUpdateQueuedTime + FTimespan::FromSeconds(Delay) < Now))
			{
				// get batch size
				const int32 MaxStatusDispatchesPerRequest = GetSourceControlFileStatusMonitor().GetMaxFilePerRequestPolicy();

				// grab a batch of files
				TArray<FString> FilesToDispatch;
				FilesToDispatch.Reserve(MaxStatusDispatchesPerRequest);
				for (const FString& Filename : PendingStatusUpdateFiles)
				{
					if(FilesToDispatch.Num() >= MaxStatusDispatchesPerRequest)
					{
						break;
					}
					FilesToDispatch.Add(Filename);
				}

				if (FilesToDispatch.Num() > 0)
				{
					// remove the files we are dispatching so we don't try again
					PendingStatusUpdateFiles.RemoveAt(0, FilesToDispatch.Num());

					// dispatch update
					Provider.Execute(ISourceControlOperation::Create<FUpdateStatus>(), FilesToDispatch, EConcurrency::Asynchronous);
				}
			}
		}
	}
}

void FSourceControlModule::QueueStatusUpdate(const TArray<UPackage*>& InPackages)
{
	if (IsEnabled())
	{
		for (UPackage* Package: InPackages)
		{
			QueueStatusUpdate(Package);
		}
	}
}

void FSourceControlModule::QueueStatusUpdate(const TArray<FString>& InFilenames)
{
	if (IsEnabled())
	{
		for (const FString& Filename : InFilenames)
		{
			QueueStatusUpdate(Filename);
		}
	}
}

void FSourceControlModule::QueueStatusUpdate(UPackage* InPackage)
{
	if (IsEnabled())
	{
		QueueStatusUpdate(SourceControlHelpers::PackageFilename(InPackage));
	}
}

void FSourceControlModule::QueueStatusUpdate(const FString& InFilename)
{
	if (IsEnabled())
	{
		TSharedPtr<ISourceControlState, ESPMode::ThreadSafe> SourceControlState = GetProvider().GetState(InFilename, EStateCacheUsage::Use);
		if (SourceControlState.IsValid())
		{
			FTimespan TimeSinceLastUpdate = FDateTime::Now() - SourceControlState->GetTimeStamp();
			if (TimeSinceLastUpdate > SourceControlConstants::StateRefreshInterval)
			{
				PendingStatusUpdateFiles.AddUnique(InFilename);
				PendingStatusUpdateQueuedTime = FDateTime::UtcNow();
			}
		}
	}
}

bool FSourceControlModule::IsEnabled() const
{
	return TemporarilyDisabledCount == 0 && GetProvider().IsEnabled();
}

ISourceControlProvider& FSourceControlModule::GetProvider() const
{
	return *CurrentSourceControlProvider;
}

TUniquePtr<ISourceControlProvider> FSourceControlModule::CreateProvider(const FName& ProviderName, const FStringView& OwnerName, const FSourceControlInitSettings& InitialSettings) const
{
	TArray<ISourceControlProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ISourceControlProvider>(SourceControlFeatureName);
	for (const ISourceControlProvider* DefaultProvider : Providers)
	{
		if (DefaultProvider->GetName() == ProviderName)
		{
			return DefaultProvider->Create(OwnerName, InitialSettings);
		}
	}

	// Provider was not found
	return TUniquePtr<ISourceControlProvider>();
}

FSourceControlAssetDataCache& FSourceControlModule::GetAssetDataCache()
{
	return AssetDataCache;
}

void FSourceControlModule::SetProvider( const FName& InName )
{
	TArray<ISourceControlProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ISourceControlProvider>(SourceControlFeatureName);
	for(auto It(Providers.CreateIterator()); It; It++)
	{
		ISourceControlProvider* Provider = *It;
		if(InName == Provider->GetName())
		{
			SetCurrentSourceControlProvider(*Provider);
			return;
		}
	}

	UE_LOGF(LogSourceControl, Fatal, "Tried to set unknown revision control provider: %ls", *InName.ToString());
}

void FSourceControlModule::ClearCurrentSourceControlProvider()
{
	if( !CurrentSourceControlProvider || CurrentSourceControlProvider != &DefaultSourceControlProvider )
	{
		ISourceControlProvider* OldSourceControlProvider = CurrentSourceControlProvider;
		if (CurrentSourceControlProvider)
		{
			CurrentSourceControlProvider->Close();
		}

		CurrentSourceControlProvider = &DefaultSourceControlProvider;

		if (OldSourceControlProvider)
		{
			OnSourceControlProviderChanged.Broadcast(*OldSourceControlProvider, *CurrentSourceControlProvider);
		}
	}
}

int32 FSourceControlModule::GetNumSourceControlProviders()
{
	return IModularFeatures::Get().GetModularFeatureImplementationCount(SourceControlFeatureName);
}

void FSourceControlModule::SetCurrentSourceControlProvider(int32 ProviderIndex)
{
	TArray<ISourceControlProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ISourceControlProvider>(SourceControlFeatureName);
	check(Providers.IsValidIndex(ProviderIndex));
	SetCurrentSourceControlProvider(*Providers[ProviderIndex]);
}

void FSourceControlModule::SetCurrentSourceControlProvider(ISourceControlProvider& InProvider)
{
	// see if we are switching or not
	if(&InProvider == CurrentSourceControlProvider)
	{
		return;
	}

	ClearCurrentSourceControlProvider();

	ISourceControlProvider* OldSourceControlProvider = CurrentSourceControlProvider;

	CurrentSourceControlProvider = &InProvider;
	CurrentSourceControlProvider->Init(false);	// Don't force a connection here, as its synchronous. Let the user establish a connection.

	SourceControlSettings.SetProvider(CurrentSourceControlProvider->GetName().ToString());

	SaveSettings();

	if (OldSourceControlProvider)
	{
		OnSourceControlProviderChanged.Broadcast(*OldSourceControlProvider, *CurrentSourceControlProvider);
	}
}

FName FSourceControlModule::GetSourceControlProviderName(int32 ProviderIndex)
{
	TArray<ISourceControlProvider*> Providers = IModularFeatures::Get().GetModularFeatureImplementations<ISourceControlProvider>(SourceControlFeatureName);
	check(Providers.IsValidIndex(ProviderIndex));
	return Providers[ProviderIndex]->GetName();
}

TSharedPtr<class SSourceControlLogin> FSourceControlModule::GetLoginWidget() const
{
	return SourceControlLoginPtr;
}

void FSourceControlModule::HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == SourceControlFeatureName)
	{
		InitializeSourceControlProviders();
	}
}

void FSourceControlModule::HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature)
{
	if(Type == SourceControlFeatureName && CurrentSourceControlProvider == static_cast<ISourceControlProvider*>(ModularFeature))
	{
		ClearCurrentSourceControlProvider();
	}
}

#if WITH_EDITOR
void FSourceControlModule::OnAssetsCanDelete(const TArray<UObject*>& InObjects, FCanDeleteAssetResult& OutCanDeleteAssetResult) const
{
	ISourceControlProvider& SourceControlProvider = GetProvider();
	if (IsEnabled() && SourceControlProvider.IsAvailable())
	{
		TArray<FString> PackagesNames;
		PackagesNames.Reserve(InObjects.Num());
		for (UObject* Object : InObjects)
		{
			if (Object)
			{
				if (UPackage* ObjectPackage = Object->GetPackage())
				{
					FString PackageFilename;
					if (FPackageName::DoesPackageExist(ObjectPackage->GetName(), &PackageFilename))
					{
						PackagesNames.Add(PackageFilename);
					}
				}
			}
		}

		TArray<FSourceControlStateRef> PackageSCCStates;
		if (PackagesNames.Num())
		{
			SourceControlProvider.GetState(PackagesNames, PackageSCCStates, EStateCacheUsage::ForceUpdate);
		}

		for (const FSourceControlStateRef& SCCState : PackageSCCStates)
		{
			if (SCCState->IsSourceControlled())
			{
				if (!SCCState->CanDelete())
				{
					OutCanDeleteAssetResult.Set(false);
					UE_LOGF(LogSourceControl, Warning, "Asset(s) can't be deleted because at least one is currently checked out by another user and/or not at latest.");
					return;
				}
			}
		}
	}
}
#endif

bool FSourceControlModule::GetUseGlobalSettings() const
{
	return SourceControlSettings.GetUseGlobalSettings();
}

void FSourceControlModule::SetUseGlobalSettings(bool bIsUseGlobalSettings)
{
	SourceControlSettings.SetUseGlobalSettings(bIsUseGlobalSettings);
	
	// force the user to re-log in
	ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modeless, EOnLoginWindowStartup::PreserveProvider);
}	

FSourceControlProviderChanging& FSourceControlModule::GetSourceControlProviderChanging()
{
	return OnSourceControlProviderChanging;
}

FDelegateHandle FSourceControlModule::RegisterProviderChanged(const FSourceControlProviderChanged::FDelegate& SourceControlProviderChanged)
{
	return OnSourceControlProviderChanged.Add(SourceControlProviderChanged);
}

void FSourceControlModule::UnregisterProviderChanged(FDelegateHandle Handle)
{
	OnSourceControlProviderChanged.Remove(Handle);
}

void FSourceControlModule::RegisterPreSubmitChangelistValidationEx(const FSourceControlPreSubmitChangelistValidationExDelegate& PreSubmitChangelistValidationDelegate)
{
	OnSourceControlPreSubmitChangelistValidation = PreSubmitChangelistValidationDelegate;
}

void FSourceControlModule::UnregisterPreSubmitChangelistValidation()
{
	OnSourceControlPreSubmitChangelistValidation = FSourceControlPreSubmitChangelistValidationExDelegate();
}

FSourceControlPreSubmitChangelistValidationExDelegate FSourceControlModule::GetRegisteredPreSubmitChangelistValidationEx()
{
	return OnSourceControlPreSubmitChangelistValidation;
}

void FSourceControlModule::RegisterPreSubmitFilesValidationEx(const FSourceControlPreSubmitFilesValidationExDelegate& PreSubmitFilesValidationDelegate)
{
	OnSourceControlPreSubmitFilesValidation = PreSubmitFilesValidationDelegate;
}

void FSourceControlModule::UnregisterPreSubmitFilesValidation()
{
	OnSourceControlPreSubmitFilesValidation = FSourceControlPreSubmitFilesValidationExDelegate();
}

FSourceControlPreSubmitFilesValidationExDelegate FSourceControlModule::GetRegisteredPreSubmitFilesValidationEx()
{
	return OnSourceControlPreSubmitFilesValidation;
}

FDelegateHandle FSourceControlModule::RegisterPreSubmitFinalize(const FSourceControlPreSubmitFinalizeDelegate::FDelegate& Delegate)
{
	return OnPresubmitFinalize.Add(Delegate);
}

void FSourceControlModule::UnregisterPreSubmitFinalize(FDelegateHandle Handle)
{
	OnPresubmitFinalize.Remove(Handle);
}

const FSourceControlPreSubmitFinalizeDelegate& FSourceControlModule::GetOnPreSubmitFinalize() const
{
	return OnPresubmitFinalize;
}

FDelegateHandle FSourceControlModule::RegisterFilesDeleted(const FSourceControlFilesDeletedDelegate::FDelegate& InDelegate)
{
	return OnFilesDeleted.Add(InDelegate);
}

void FSourceControlModule::UnregisterFilesDeleted(FDelegateHandle InHandle)
{
	OnFilesDeleted.Remove(InHandle);
}

const FSourceControlFilesDeletedDelegate& FSourceControlModule::GetOnFilesDeleted() const
{
	return OnFilesDeleted;
}

void FSourceControlModule::RegisterCustomProjectsDelegate(FSourceControlCustomProjectsDelegate InCustomProjectsDelegate)
{
	CustomProjectsDelegate = MoveTemp(InCustomProjectsDelegate);
	OnCustomProjectsChanged().Broadcast();
}

void FSourceControlModule::UnregisterCustomProjectsDelegate()
{
	CustomProjectsDelegate = FSourceControlCustomProjectsDelegate();
	OnCustomProjectsChanged().Broadcast();
}

TArray<FSourceControlProjectInfo> FSourceControlModule::GetCustomProjects() const
{
	return CustomProjectsDelegate.IsBound() ? CustomProjectsDelegate.Execute() : TArray<FSourceControlProjectInfo>();
}

FSimpleMulticastDelegate& FSourceControlModule::OnCustomProjectsChanged()
{
	return CustomProjectsChangedDelegate;
}

FString FSourceControlModule::GetSourceControlProjectDir() const
{
	{
		TArray<FSourceControlProjectInfo> CustomProjects = GetCustomProjects();
		if (!CustomProjects.IsEmpty())
		{
			return CustomProjects[0].ProjectDirectory;
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (SourceControlProjectDirDelegate.IsBound())
	{
		FString ProjectDir = SourceControlProjectDirDelegate.Execute();
		if (!ProjectDir.IsEmpty())
		{
			return ProjectDir;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return FPaths::ProjectDir();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FSourceControlModule::RegisterSourceControlProjectDirDelegate(const FSourceControlProjectDirDelegate& InSourceControlProjectDirDelegate)
{
	SourceControlProjectDirDelegate = InSourceControlProjectDirDelegate;
}

void FSourceControlModule::UnregisterSourceControlProjectDirDelegate()
{
	SourceControlProjectDirDelegate = FSourceControlProjectDirDelegate();
}

bool FSourceControlModule::UsesCustomProjectDir() const
{
	if (CustomProjectsDelegate.IsBound())
	{
		TArray<FSourceControlProjectInfo> Projects = CustomProjectsDelegate.Execute();
		if (!Projects.IsEmpty())
		{
			return true;
		}
	}
	else if (SourceControlProjectDirDelegate.IsBound())
	{
		FString ProjectDir = SourceControlProjectDirDelegate.Execute();
		if (!ProjectDir.IsEmpty())
		{
			return true;
		}
	}
	return false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FSourceControlFileStatusMonitor& FSourceControlModule::GetSourceControlFileStatusMonitor()
{
	return *SourceControlFileStatusMonitor;
}

IMPLEMENT_MODULE( FSourceControlModule, SourceControl );

#undef LOCTEXT_NAMESPACE
