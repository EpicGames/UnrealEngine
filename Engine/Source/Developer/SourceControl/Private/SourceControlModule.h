// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/**
 * Interface for talking to source control clients
 */

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ISourceControlModule.h"
#include "SourceControlAssetDataCache.h"
#include "SourceControlSettings.h"
#include "DefaultSourceControlProvider.h"

class FSourceControlFileStatusMonitor;
class SSourceControlLogin;
class SWindow;

struct FCanDeleteAssetResult;

class FSourceControlModule final : public ISourceControlModule
{
public:
	FSourceControlModule();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** ISourceControlModule implementation */
	virtual void GetProviderNames(TArray<FName>& OutProviderNames) override;
	virtual void Tick() override;
	virtual void QueueStatusUpdate(const TArray<UPackage*>& InPackages) override;
	virtual void QueueStatusUpdate(const TArray<FString>& InFilenames) override;
	virtual void QueueStatusUpdate(UPackage* InPackage) override;
	virtual void QueueStatusUpdate(const FString& InFilename) override;
	virtual bool IsEnabled() const override;
	virtual ISourceControlProvider& GetProvider() const override;
	virtual TUniquePtr<ISourceControlProvider> CreateProvider(const FName& ProviderName, const FStringView& OwnerName, const FSourceControlInitSettings& InitialSettings) const override;
	virtual FSourceControlAssetDataCache& GetAssetDataCache() override;
	virtual void SetProvider( const FName& InName ) override;
	virtual void ShowLoginDialog(const FSourceControlLoginClosed& InOnSourceControlLoginClosed, ELoginWindowMode::Type InLoginWindowMode, EOnLoginWindowStartup::Type InOnLoginWindowStartup = EOnLoginWindowStartup::ResetProviderToNone) override;
	virtual bool GetUseGlobalSettings() const override;
	virtual void SetUseGlobalSettings(bool bIsUseGlobalSettings) override;
	virtual FSourceControlProviderChanging& GetSourceControlProviderChanging() override;
	virtual FDelegateHandle RegisterProviderChanged(const FSourceControlProviderChanged::FDelegate& SourceControlProviderChanged) override;
	virtual void UnregisterProviderChanged(FDelegateHandle Handle) override;
	
	virtual void RegisterPreSubmitChangelistValidationEx(const FSourceControlPreSubmitChangelistValidationExDelegate& PreSubmitChangelistValidationDelegate) override;
	virtual void UnregisterPreSubmitChangelistValidation() override;
	virtual FSourceControlPreSubmitChangelistValidationExDelegate GetRegisteredPreSubmitChangelistValidationEx() override;
	
	virtual void RegisterPreSubmitFilesValidationEx(const FSourceControlPreSubmitFilesValidationExDelegate& PreSubmitFilesValidationDelegate) override;
	virtual void UnregisterPreSubmitFilesValidation() override;
	virtual FSourceControlPreSubmitFilesValidationExDelegate GetRegisteredPreSubmitFilesValidationEx() override;
	
	virtual FDelegateHandle RegisterPreSubmitFinalize(const FSourceControlPreSubmitFinalizeDelegate::FDelegate& Delegate) override;
	virtual void UnregisterPreSubmitFinalize(FDelegateHandle Handle) override;
	virtual const FSourceControlPreSubmitFinalizeDelegate& GetOnPreSubmitFinalize() const override;

	virtual FDelegateHandle RegisterFilesDeleted(const FSourceControlFilesDeletedDelegate::FDelegate& InDelegate) override;
	virtual void UnregisterFilesDeleted(FDelegateHandle InHandle) override;
	virtual const FSourceControlFilesDeletedDelegate& GetOnFilesDeleted() const override;

	virtual void RegisterCustomProjectsDelegate(FSourceControlCustomProjectsDelegate InCustomProjectsDelegate) override;
	virtual void UnregisterCustomProjectsDelegate() override;
	virtual TArray<FSourceControlProjectInfo> GetCustomProjects() const override;
	virtual FSimpleMulticastDelegate& OnCustomProjectsChanged() override;
	virtual FString GetSourceControlProjectDir() const override;

	UE_DEPRECATED(5.5, "Use RegisterCustomProjectsDelegate instead.")
	virtual void RegisterSourceControlProjectDirDelegate(const FSourceControlProjectDirDelegate& SourceControlProjectDirDelegate) override;
	UE_DEPRECATED(5.5, "Use UnregisterCustomProjectsDelegate instead.")
	virtual void UnregisterSourceControlProjectDirDelegate() override;
	UE_DEPRECATED(5.5, "Use !GetCustomProjects().IsEmpty() instead.")
	virtual bool UsesCustomProjectDir() const override;

	virtual FSourceControlFileStatusMonitor& GetSourceControlFileStatusMonitor() override;

	/** Save the settings to the ini file */
	void SaveSettings();

	/**
	 * Get the number of currently registered source control providers.
	 */
	int32 GetNumSourceControlProviders();

	/**
	 * Set the current source control provider by index.
	 */
	void SetCurrentSourceControlProvider(int32 ProviderIndex);

	/**
	 * Get the name of the source control provider at the specified index.
	 */
	FName GetSourceControlProviderName(int32 ProviderIndex);

	/**
	 * Get the one and only login widget, if any.
	 */
	TSharedPtr<class SSourceControlLogin> GetLoginWidget() const;

	/**
	 * Gets a reference to the source control module instance.
	 *
	 * @return A reference to the source control module.
	 */
	static FSourceControlModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FSourceControlModule>("SourceControl");
	}

private:
	/** Refresh & initialize the current source control provider */
	void InitializeSourceControlProviders();

	/** Close the current source control provider & set the current to default - 'None' */
	void ClearCurrentSourceControlProvider();

	/** Set the current source control provider to the passed-in value */
	void SetCurrentSourceControlProvider(ISourceControlProvider& InProvider);

	/** Delegate called when the source control window is closed */
	void OnSourceControlDialogClosed(const TSharedRef<class SWindow>& InWindow);

	/** Delegate handling when source control features are registered */
	void HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature);

	/** Delegate handling when source control features are unregistered */
	void HandleModularFeatureUnregistered(const FName& Type, IModularFeature* ModularFeature);

#if WITH_EDITOR
	/** Delegate called when trying to delete asset(s) to verify that it is allowed by the SourceControl */
	void OnAssetsCanDelete(const TArray<UObject*>& InObjects, FCanDeleteAssetResult& OutCanDeleteAssetResult) const;
#endif
private:
	/** The settings object */
	FSourceControlSettings SourceControlSettings;

	/** Current source control provider */
	ISourceControlProvider* CurrentSourceControlProvider;

	/** Source control provider we use if there are none registered */
	FDefaultSourceControlProvider DefaultSourceControlProvider;

	/** The login window we may be using */
	TSharedPtr<SWindow> SourceControlLoginWindowPtr;

	/** The login window control we may be using */
	TSharedPtr<class SSourceControlLogin> SourceControlLoginPtr;

	/** Monitor the source control status of a collection of files. */
	TSharedPtr<class FSourceControlFileStatusMonitor> SourceControlFileStatusMonitor;

	/** Files pending a status update */
	TArray<FString> PendingStatusUpdateFiles;

	/** Time last status update got queued */
	FDateTime PendingStatusUpdateQueuedTime;

	/** Flag to disable source control - used temporarily when login is in progress */
	uint8 TemporarilyDisabledCount;

	/** Active Provider name to track source control provider changes */
	FString ActiveProviderName;

	/** For notifying when the source provider is about to change */
	FSourceControlProviderChanging OnSourceControlProviderChanging;

	/** For notifying when the source provider is changed */
	FSourceControlProviderChanged OnSourceControlProviderChanged;

	/** To call when doing pre-submit changelist validation */
	FSourceControlPreSubmitChangelistValidationExDelegate OnSourceControlPreSubmitChangelistValidation;

	/** To call when doing pre-submit files validation */
	FSourceControlPreSubmitFilesValidationExDelegate OnSourceControlPreSubmitFilesValidation;
	
	/** To be called right before files are submitted, allowing for additional last minute validation. @see FSourceControlPreSubmitFinalizeDelegate */
	FSourceControlPreSubmitFinalizeDelegate OnPresubmitFinalize;

	/** To be called after a source control operations deleted files */
	FSourceControlFilesDeletedDelegate OnFilesDeleted;

	/** Used to cache source controlled AssetData information */
	FSourceControlAssetDataCache AssetDataCache;

	UE_DEPRECATED(5.5, "Use CustomProjectsDelegate instead.")
	FSourceControlProjectDirDelegate SourceControlProjectDirDelegate;

	/** Delegate used to return information on custom projects under source control */
	FSourceControlCustomProjectsDelegate CustomProjectsDelegate;
	FSimpleMulticastDelegate CustomProjectsChangedDelegate;

	friend FScopedDisableSourceControl;
};
