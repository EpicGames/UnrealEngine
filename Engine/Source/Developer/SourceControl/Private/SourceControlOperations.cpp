// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlOperations.h"

#include "Containers/StringView.h"
#include "HAL/FileManager.h"
#include "ISourceControlModule.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"

bool FUpdateStatus::IsDirectoryPath(const FString& Path) const
{
	// Problem:
	// We need to be able to work out if a path if a directory or not as some source 
	// control implementations will need to append wildcards to the path if it is
	// a directory in order for it to function correctly.
	// 
	// Strictly speaking the only way we can tell the difference between a file path
	// and a directory path is to check with the file system. 
	// However some source control implementations let us reference files on the server
	// and do not require us to have them on the users machine. This means checking with
	// the file system does not give an accurate result and can end up being very 
	// slow if the file system interprets the path as a UNC path and tries to resolve it.
	//
	// Ideally we would let the caller clearly mark if the path is for a file or a 
	// directory but this would require a large overhaul of the source control API.
	// We cannot change the existing behavior as it is very likely that 3rd party code is
	// relying on it.
	//
	// Solution:
	// As a 'temporary' solution, the caller can set bSetRequireDirPathEndWithSeparator
	// to true. When it is true we will assume that all directory paths end with a 
	// separator. Although we tend to not terminate directory paths with a separator in the 
	// engine, this will allow us an opt in to the new behavior until more substantial API
	// changes can be made.

	if (bSetRequireDirPathEndWithSeparator)
	{
		return Path.EndsWith(TEXT("/")) || Path.EndsWith(TEXT("\\"));
	}
	else
	{
		return IFileManager::Get().DirectoryExists(*Path);
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FDownloadFile::FDownloadFile(FStringView InTargetDirectory, EVerbosity InVerbosity)
	: FDownloadFile(InTargetDirectory)
{
	if (InVerbosity == EVerbosity::None)
	{
		SetEnableErrorLogging(false);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FDownloadFile::FDownloadFile(FStringView InTargetDirectory)
	: TargetDirectory(InTargetDirectory)
{
	FPaths::NormalizeDirectoryName(TargetDirectory);

	// Due to the asynchronous nature of the source control api, it might be some time before
	// the TargetDirectory is actually used. So we do a validation pass on it now to try and 
	// give errors close to the point that they were created. The caller is still free to try
	// and use the FDownloadFile but with an invalid path it probably won't work.
	FText Reason;
	if (!FPaths::ValidatePath(TargetDirectory, &Reason))
	{
		UE_LOGF(LogSourceControl, Error, "Path '%ls' passed to FDownloadFile is invalid due to: %ls", 
			*TargetDirectory, *Reason.ToString());
	}
}

FSharedBuffer FDownloadFile::GetFileData(const FStringView& Filename)
{
	const uint32 Hash = GetTypeHash(Filename);
	FSharedBuffer* Buffer = FileDataMap.FindByHash(Hash, Filename);
	if (Buffer != nullptr)
	{
		return *Buffer;
	}
	else
	{
		return FSharedBuffer();
	}
}

FCreateWorkspace::FCreateWorkspace(FStringView InWorkspaceName, FStringView InWorkspaceRoot)
	: WorkspaceName(InWorkspaceName)
{
	TStringBuilder<512> AbsoluteWorkspaceName;
	FPathViews::ToAbsolutePath(InWorkspaceRoot, AbsoluteWorkspaceName);

	WorkspaceRoot = AbsoluteWorkspaceName.ToString();
}
