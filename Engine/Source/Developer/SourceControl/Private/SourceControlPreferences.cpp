// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlPreferences.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SourceControlPreferences)

bool USourceControlPreferences::IsValidationTagEnabled()
{
	return GetDefault<USourceControlPreferences>()->bEnableValidationTag;
}

bool USourceControlPreferences::ShouldDeleteNewFilesOnRevert()
{
	return GetDefault<USourceControlPreferences>()->bShouldDeleteNewFilesOnRevert;
}

bool USourceControlPreferences::AreUncontrolledChangelistsEnabled()
{
	return GetDefault<USourceControlPreferences>()->bEnableUncontrolledChangelists;
}

bool USourceControlPreferences::RequiresRevisionControlToRenameLocalizableAssets()
{
	// This feature only works for Perforce (assume it is Perforce until Revision Control is configured)
	bool bRevisionControlIsPerforce = true;
	ISourceControlModule& SCModule = ISourceControlModule::Get();
	if (SCModule.IsEnabled())
	{
		ISourceControlProvider& SCProvider = SCModule.GetProvider();
		bRevisionControlIsPerforce = SCProvider.GetName() == "Perforce";
	}

	return bRevisionControlIsPerforce && GetDefault<USourceControlPreferences>()->bRequiresRevisionControlToRenameLocalizableAssets;
}

