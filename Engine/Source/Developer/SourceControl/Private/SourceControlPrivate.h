// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"

#ifndef SOURCE_CONTROL_WITH_SLATE
	#error SOURCE_CONTROL_WITH_SLATE not defined
#endif // SOURCE_CONTROL_WITH_SLATE

#if SOURCE_CONTROL_WITH_SLATE
	#include "SlateBasics.h"
	#include "Styling/AppStyle.h"
#endif // SOURCE_CONTROL_WITH_SLATE

#if WITH_EDITOR
#include "PackageTools.h"
#include "Logging/MessageLog.h"
#endif

#if WITH_UNREAL_DEVELOPER_TOOLS
#include "MessageLogModule.h"
#endif
