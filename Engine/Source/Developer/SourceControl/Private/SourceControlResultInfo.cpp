// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlResultInfo.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "SourceControlResultInfo"

void FSourceControlResultInfo::OnConnectionFailed()
{
	ErrorMessages.Add(LOCTEXT("SC_ConnectionFailed", "Failed to connect to the server"));
	AdditionalErrorContext |= EAdditionalErrorContext::ConnectionFailed;
}

void FSourceControlResultInfo::OnConnectionDroped()
{
	ErrorMessages.Add(LOCTEXT("SC_ConnectionDropped", "Connection to the server dropped"));
	AdditionalErrorContext |= EAdditionalErrorContext::ConnectionDropped;
}

bool FSourceControlResultInfo::DidConnectionFail() const
{
	return EnumHasAnyFlags(AdditionalErrorContext,	EAdditionalErrorContext::ConnectionFailed |
													EAdditionalErrorContext::ConnectionDropped);
}

#undef LOCTEXT_NAMESPACE
