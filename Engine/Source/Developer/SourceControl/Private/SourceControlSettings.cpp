// Copyright Epic Games, Inc. All Rights Reserved.

#include "SourceControlSettings.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "SourceControlHelpers.h"

namespace SourceControlSettingsConstants
{

/** The section of the ini file we load our settings from */
static const FString SettingsSection = TEXT("SourceControl.SourceControlSettings");

}


const FString& FSourceControlSettings::GetProvider() const
{
	return Provider;
}

void FSourceControlSettings::SetProvider(const FString& InString)
{
	Provider = InString;
}

bool FSourceControlSettings::GetUseGlobalSettings() const
{
	return bUseGlobalSettings;
}

void FSourceControlSettings::SetUseGlobalSettings(bool bInUseGlobalSettings)
{
	bUseGlobalSettings = bInUseGlobalSettings;
}

void FSourceControlSettings::LoadSettings()
{
	// make sure we load the global ini first
	const FString& GlobalIniFile = SourceControlHelpers::GetGlobalSettingsIni();
	GConfig->GetBool(*SourceControlSettingsConstants::SettingsSection, TEXT("UseGlobalSettings"), bUseGlobalSettings, GlobalIniFile);

	TArray<FString> Tokens;
	TArray<FString> Switches;
	FCommandLine::Parse( FCommandLine::Get(), Tokens, Switches );
	TMap<FString, FString> SwitchPairs;
	for (int32 SwitchIdx = Switches.Num() - 1; SwitchIdx >= 0; --SwitchIdx)
	{
		FString& Switch = Switches[SwitchIdx];
		TArray<FString> SplitSwitch;
		if (2 == Switch.ParseIntoArray(SplitSwitch, TEXT("="), true))
		{
			SwitchPairs.Add(SplitSwitch[0], SplitSwitch[1].TrimQuotes());
			Switches.RemoveAt(SwitchIdx);
		}
	}

	if( SwitchPairs.Contains( TEXT("SCCProvider") ) )
	{
		Provider = SwitchPairs[TEXT("SCCProvider")];
	}
	else
	{
		const FString& IniFile = SourceControlHelpers::GetSettingsIni();
		GConfig->GetString(*SourceControlSettingsConstants::SettingsSection, TEXT("Provider"), Provider, IniFile);
	}
}

void FSourceControlSettings::SaveSettings() const
{
	const FString& IniFile = SourceControlHelpers::GetSettingsIni();
	const FString& GlobalIniFile = SourceControlHelpers::GetGlobalSettingsIni();
	GConfig->SetString(*SourceControlSettingsConstants::SettingsSection, TEXT("Provider"), *Provider, IniFile);
	GConfig->SetBool(*SourceControlSettingsConstants::SettingsSection, TEXT("UseGlobalSettings"), bUseGlobalSettings, GlobalIniFile);	
}
