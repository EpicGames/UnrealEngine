// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FSourceControlSettings
{
public:
	FSourceControlSettings()
		: bUseGlobalSettings(false)
	{
	}

	/** Get the provider we want to use */
	const FString& GetProvider() const;

	/** Set the provider we want to use */
	void SetProvider(const FString& InString);

	/** Get whether we should use global or per-project settings */
	bool GetUseGlobalSettings() const;

	/** Set whether we should use global or per-project settings */
	void SetUseGlobalSettings(bool bInUseGlobalSettings);

	/** Load settings from ini file */
	void LoadSettings();

	/** Save settings to ini file */
	void SaveSettings() const;

private:
	/** The preferred source control provider */
	FString Provider;

	/** Whether we should use global or per-project settings */
	bool bUseGlobalSettings;
};
