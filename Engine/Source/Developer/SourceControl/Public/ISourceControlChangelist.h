// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class ISourceControlChangelist;

typedef TSharedRef<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistRef;
typedef TSharedPtr<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistPtr;

/**
 * An abstraction of a changelist under source control
 */
class ISourceControlChangelist : public TSharedFromThis<ISourceControlChangelist, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlChangelist() = default;

	/**
	 * Returns true if the changelist is deletable. Some special changelist might not be deletable like the
	 * default Perforce changelist.
	 */
	virtual bool CanDelete() const { return true; }

	/**
	 * Get source control specific identifier for this changelist, for example CL number in Perforce.
	 */
	virtual FString GetIdentifier() const { return TEXT(""); }

	/**
	 * Returns true if this changelist is a default changelist.
	 */
	virtual bool IsDefault() const { return false; }
};
