// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlState.h"
#include "ISourceControlChangelist.h"

typedef TSharedRef<class ISourceControlChangelistState, ESPMode::ThreadSafe> FSourceControlChangelistStateRef;
typedef TSharedPtr<class ISourceControlChangelistState, ESPMode::ThreadSafe> FSourceControlChangelistStatePtr;

/**
 * An abstraction of the state of a pending changelist under source control
 */
class ISourceControlChangelistState : public TSharedFromThis<ISourceControlChangelistState, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlChangelistState() = default;

	/**
	 * Get the name of the icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetIconName() const = 0;

	/**
	 * Get the name of the small icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	virtual FName GetSmallIconName() const = 0;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDisplayText() const = 0;

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDescriptionText() const = 0;

	/**
	 * Returns whether the change list description can be saved and persisted. Some changelists are
	 * special and the description cannot be persisted with the changelist. For example the P4 default
	 * changelist description cannot be persisted and the user must create a new changelist.
	 */
	virtual bool SupportsPersistentDescription() const { return true; }

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	virtual FText GetDisplayTooltip() const = 0;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	virtual const FDateTime& GetTimeStamp() const = 0;

	virtual const TArray<FSourceControlStateRef> GetFilesStates() const = 0;
	virtual int32 GetFilesStatesNum() const = 0;

	virtual const TArray<FSourceControlStateRef> GetShelvedFilesStates() const = 0;
	virtual int32 GetShelvedFilesStatesNum() const = 0;

	/**
	 * Returns the object on which this state was constructed
	 * @returns the changelist associated to this state
	 */
	virtual FSourceControlChangelistRef GetChangelist() const = 0;
};
