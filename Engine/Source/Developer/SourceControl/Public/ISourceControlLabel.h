// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** 
 * Abstraction of a source control label/tag.
 */
class ISourceControlLabel : public TSharedFromThis<ISourceControlLabel>
{
public:

	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlLabel() = default;

	/**
	 * Get the name of this label
	 */
	virtual const FString& GetName() const = 0;

	/**
	 * Get a list of file revisions at this label.
	 * @param	InFile			The file/directory to get
	 * @param	OutRevisions	The revisions retrieved by the operation
	 */
	virtual bool GetFileRevisions( const FString& InFile, TArray< TSharedRef<class ISourceControlRevision, ESPMode::ThreadSafe> >& OutRevisions ) const
	{
		TArray<FString> Files;
		Files.Add(InFile);
		return GetFileRevisions( Files, OutRevisions );
	}

	/**
	 * Get a list of file revisions at this label.
	 * @param	InFiles			The files/directories to get
	 * @param	OutRevisions	The revisions retrieved by the operation
	 */
	virtual bool GetFileRevisions( const TArray<FString>& InFiles, TArray< TSharedRef<class ISourceControlRevision, ESPMode::ThreadSafe> >& OutRevisions ) const = 0;

	/**
	 * Sync a path/filename to this label
	 * @param	InFilename	The path or filename to sync
	 */
	virtual bool Sync( const FString& InFilename ) const
	{
		TArray<FString> Files;
		Files.Add(InFilename);
		return Sync( Files );
	}

	/**
	 * Sync a list of paths/filenames to this label
	 * @param	InFilenames	The paths or filenames to sync
	 */
	virtual bool Sync( const TArray<FString>& InFilenames ) const = 0;
};
