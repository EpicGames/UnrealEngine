// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/UObjectGlobals.h"

class ISourceControlProvider;
class ISourceControlChangelist;
class FSourceControlFileStatusMonitor;

class FSourceControlAssetDataCache;
class FSourceControlInitSettings;

typedef TSharedPtr<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistPtr;
typedef TSharedRef<class ISourceControlState, ESPMode::ThreadSafe> FSourceControlStateRef;

/**
 * Describes a custom project that is under source control
 */
struct FSourceControlProjectInfo
{
	FSourceControlProjectInfo()
	{}

	FSourceControlProjectInfo(FString InProjectDirectory)
		: ProjectDirectory(MoveTemp(InProjectDirectory))
	{}

	friend bool operator==(const FSourceControlProjectInfo& A, const FSourceControlProjectInfo& B)
	{
		return A.ProjectDirectory == B.ProjectDirectory
			&& A.ContentDirectories == B.ContentDirectories;
	}

	friend bool operator!=(const FSourceControlProjectInfo& A, const FSourceControlProjectInfo& B)
	{
		return !(A == B);
	}

	/** Directory that contains project files */
	FString ProjectDirectory;

	/** Content directories that contain asset files */
	TArray<FString> ContentDirectories;
};

SOURCECONTROL_API DECLARE_LOG_CATEGORY_EXTERN(LogSourceControl, Log, All);

/** Delegate called when the source control login window is closed. Parameter determines if source control is enabled or not */
DECLARE_DELEGATE_OneParam( FSourceControlLoginClosed, bool );

/** Delegate called when the active source control provider is changing. Returns false if the operation is to be aborted */
DECLARE_DELEGATE_RetVal(bool, FSourceControlProviderChanging);

/** Delegate called when the active source control provider is changed */
DECLARE_MULTICAST_DELEGATE_TwoParams( FSourceControlProviderChanged, ISourceControlProvider& /*OldProvider*/, ISourceControlProvider& /*NewProvider*/ );

/** Deprecated delegates called on pre-submit for data validation */
DECLARE_DELEGATE_FourParams(FSourceControlPreSubmitDataValidationDelegate, FSourceControlChangelistPtr /*Changelist*/, EDataValidationResult& /*Result*/, TArray<FText>& /*ValidationErrors*/, TArray<FText>& /*ValidationWarnings*/);
DECLARE_DELEGATE_FourParams(FSourceControlPreSubmitChangelistValidationDelegate, FSourceControlChangelistPtr /*Changelist*/, EDataValidationResult& /*Result*/, TArray<FText>& /*ValidationErrors*/, TArray<FText>& /*ValidationWarnings*/);
DECLARE_DELEGATE_FourParams(FSourceControlPreSubmitFilesValidationDelegate, TArray<FSourceControlStateRef>& /*States*/, EDataValidationResult& /*Result*/, TArray<FText>& /*ValidationErrors*/, TArray<FText>& /*ValidationWarnings*/);

/** Structure to hold result of pre-submit data validation. */
struct FSourceControlValidationDelegateResult
{
	/** Validation result. */
	EDataValidationResult Result = EDataValidationResult::NotValidated;
	/** Error messages output during validation. */
	TArray<FText> Errors;
	/** Warning messages output during validation. */
	TArray<FText> Warnings;
	/** All messages (including error and warnings output during validation. */
	TArray<FText> AllMessages;
};

/** Delegates called on pre-submit for data validation. */
DECLARE_DELEGATE_TwoParams(FSourceControlPreSubmitChangelistValidationExDelegate, FSourceControlChangelistPtr /*Changelist*/, FSourceControlValidationDelegateResult& /*Result*/);
DECLARE_DELEGATE_TwoParams(FSourceControlPreSubmitFilesValidationExDelegate, TArray<FSourceControlStateRef>& /*States*/, FSourceControlValidationDelegateResult& /*Result*/);

/** Delegate that returns directory information on custom projects under source control (first entry is the main project) */
DECLARE_DELEGATE_RetVal(TArray<FSourceControlProjectInfo>, FSourceControlCustomProjectsDelegate);

/** Delegate used to specify the project base directory to be used by the source control */
DECLARE_DELEGATE_RetVal(FString, FSourceControlProjectDirDelegate);

/** 
 * Delegate called once the user has confirmed that they want to submit files to source control BUT before the files are actually submitted.
 * It is intended for last minute checks that can only run once there is no chance of the user canceling the actual submit.
 * At this point the only way to prevent the files from being submitted is for this delegate to return errors.
 * 
 * @param FilesToSubmit			The absolute file paths of the files being submitted
 * @param OutDescriptionTags	Lines of text to be appending to the submit description
 * @param OutErrors				Errors encountered while the delegate is invoked. If the array contains entries then files will not be submitted and the user given the errors instead
 */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FSourceControlPreSubmitFinalizeDelegate, const TArray<FString>& /*FilesToSubmit*/, TArray<FText>& /*OutDescriptionTags*/, TArray<FText>& /*OutErrors*/);

/**
 * Delegate called after source control operations deleted files.
 *
 * @param DeletedFiles			The absolute file paths of the deleted files
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FSourceControlFilesDeletedDelegate, const TArray<FString>& /*InDeletedFiles*/);

/**
 * The modality of the login window.
 */
namespace ELoginWindowMode
{
	enum Type
	{
		Modal,
		Modeless
	};
};


/**
 * Login window startup behavior
 */
namespace EOnLoginWindowStartup
{
	enum Type
	{
		ResetProviderToNone,
		PreserveProvider
	};
};

/**
 * Temporarily disable source control for the lifetime of this scoped object
 * 
 * @note While this is on scope, ISourceControlModule::IsEnabled() will return false, 
 *       but ISourceControlModule::GetProvider() will still return the current provider
 */
struct FScopedDisableSourceControl
{
public:
	SOURCECONTROL_API explicit FScopedDisableSourceControl(const bool InDisable = true);
	SOURCECONTROL_API ~FScopedDisableSourceControl();

	UE_NONCOPYABLE(FScopedDisableSourceControl);

private:
	bool bDisabled;
};

/**
 * Interface for talking to source control providers
 */
class ISourceControlModule : public IModuleInterface
{
public:

	/**
	 * Returns a list of the registered source control providers
	 */
	virtual void GetProviderNames(TArray<FName>& OutProviderNames) = 0;

	/**
	 * Tick the source control module.
	 * This is responsible for dispatching batched/queued status requests & for calling ISourceControlProvider::Tick()
	 */
	virtual void Tick() = 0;

	/**
	 * Queues a file to have its source control status updated in the background.
	 * @param	InPackages	The packages to queue.
	 */
	virtual void QueueStatusUpdate(const TArray<UPackage*>& InPackages) = 0;

	/**
	 * Queues a file to have its source control status updated in the background.
	 * @param	InFilenames	The files to queue.
	 */
	virtual void QueueStatusUpdate(const TArray<FString>& InFilenames) = 0;

	/**
	 * Queues a file to have its source control status updated in the background.
	 * @param	InPackage	The package to queue.
	 */
	virtual void QueueStatusUpdate(UPackage* InPackage) = 0;

	/**
	 * Queues a file to have its source control status updated in the background.
	 * @param	InFilename	The file to queue.
	 */
	virtual void QueueStatusUpdate(const FString& InFilename) = 0;

	/**
	 * Check whether source control is enabled.	Specifically, it returns true if a source control provider is set (regardless of whether the provider is available) and false if no provider is set.
	 */
	virtual bool IsEnabled() const = 0;

	/**
	 * Get the source control provider that is currently in use.
	 */
	virtual ISourceControlProvider& GetProvider() const = 0;

	/** 
	 * Creates and returns a unique source control provider that will be owned by
	 * the caller. This allows subsystems to own their own source control connection
	 * in addition to the general connection used by the rest of the process. This 
	 * could be to avoid the user changing settings that the subsystem needs, or so
	 * that it can open a connection with an entirely different source control type
	 * if required.
	 * 
	 * @param	Name			The name of the type of provider to create.
	 * @param	OwningSystem	The name of the system that will own the provider. This is
	 *							used to customize things, like the config section that
	 *							that providers settings will be saved to etc.
	 * 
	 * @return	A pointer to the newly created provider, this can be null if creating the 
	 *			name is invalid or if the provider type does not support unique connection
	 *			creation.
	 */
	virtual TUniquePtr<ISourceControlProvider> CreateProvider(const FName& Name, const FStringView& OwningSystem, const FSourceControlInitSettings& InitialSettings) const = 0;

	/**
	 * Get the source control AssetData information cache.
	 */
	virtual FSourceControlAssetDataCache& GetAssetDataCache() = 0;

	/**
	 * Set the current source control provider to the one specified here by name.
	 * This will assert if the provider does not exist.
	 * @param	InName	The name of the provider
	 */
	virtual void SetProvider( const FName& InName ) = 0;

	/**
	 * Show the source control login dialog
	 * @param	InOnWindowClosed		Delegate to be called when the login window is closed.
	 * @param	InLoginWindowMode		Whether the dialog should be presented modally or not. Note that this function blocks if the modality is Modal.
	 * @param	InOnLoginWindowStartup	Whether the provider should be set to 'None' on dialog startup
	 */
	virtual void ShowLoginDialog(const FSourceControlLoginClosed& InOnSourceControlLoginClosed, ELoginWindowMode::Type InLoginWindowMode, EOnLoginWindowStartup::Type InOnLoginWindowStartup = EOnLoginWindowStartup::ResetProviderToNone) = 0;

	/**
	 * Get whether we should use global or per-project settings
	 * @return true if we should use global settings
	 */
	virtual bool GetUseGlobalSettings() const = 0;

	/**
	 * Set whether we should use global or per-project settings
	 * @param bIsUseGlobalSettings	Whether we should use global settings
	 */
	virtual void SetUseGlobalSettings(bool bIsUseGlobalSettings) = 0;

	/**
	 * Retrieve a reference to a delegate to be called before the source control provider changes
	 */
	virtual FSourceControlProviderChanging& GetSourceControlProviderChanging() = 0;
	
	/**
	 * Register a delegate to be called when the source control provider changes
	 */
	virtual FDelegateHandle RegisterProviderChanged(const FSourceControlProviderChanged::FDelegate& SourceControlProviderChanged) = 0;

	/**
	 * Unregister a delegate to be called when the source control provider changes
	 */
	virtual void UnregisterProviderChanged(FDelegateHandle Handle) = 0;

	// Deprecating pure virtuals using final keyword to catch derived implementations.
	UE_DEPRECATED(5.7, "Use RegisterPreSubmitChangelistValidationEx instead.")
	virtual void RegisterPreSubmitDataValidation(const FSourceControlPreSubmitDataValidationDelegate& PreSubmitDataValidationDelegate) final {};
	UE_DEPRECATED(5.8, "Use RegisterPreSubmitChangelistValidationEx instead.")
	virtual void RegisterPreSubmitChangelistValidation(const FSourceControlPreSubmitChangelistValidationDelegate& PreSubmitChangelistValidationDelegate) final {};
	UE_DEPRECATED(5.8, "Use RegisterPreSubmitFilesValidationEx instead.")
	virtual void RegisterPreSubmitFilesValidation(const FSourceControlPreSubmitFilesValidationDelegate& PreSubmitFilesValidationDelegate) final {};
	UE_DEPRECATED(5.7, "Use UnregisterPreSubmitChangelistValidation instead.")
	virtual void UnregisterPreSubmitDataValidation() final {};
	UE_DEPRECATED(5.7, "Use GetRegisteredPreSubmitChangelistValidation instead.")
	virtual FSourceControlPreSubmitDataValidationDelegate GetRegisteredPreSubmitDataValidation() final { return FSourceControlPreSubmitDataValidationDelegate(); };
	UE_DEPRECATED(5.8, "Use GetRegisteredPreSubmitChangelistValidationEx instead.")
	virtual FSourceControlPreSubmitChangelistValidationDelegate GetRegisteredPreSubmitChangelistValidation() final { return FSourceControlPreSubmitChangelistValidationDelegate(); };
	UE_DEPRECATED(5.8, "Use GetRegisteredPreSubmitFilesValidationEx instead.")
	virtual FSourceControlPreSubmitFilesValidationDelegate GetRegisteredPreSubmitFilesValidation() final { return FSourceControlPreSubmitFilesValidationDelegate(); };

	/**
	 * Register a delegate to be called to validate asset changes by changelist before submitting changes
	 */
	virtual void RegisterPreSubmitChangelistValidationEx(const FSourceControlPreSubmitChangelistValidationExDelegate& PreSubmitChangelistValidationDelegate) = 0;

	/**
	 * Unregister a delegate called before submitting changes
	 */
	virtual void UnregisterPreSubmitChangelistValidation() = 0;

	/**
	 * Gets currently registered delegates for pre-submit changelist validation
	 */
	virtual FSourceControlPreSubmitChangelistValidationExDelegate GetRegisteredPreSubmitChangelistValidationEx() = 0;

	/**
	 * Register a delegate to be called to validate asset changes by files before submitting changes
	 */
	virtual void RegisterPreSubmitFilesValidationEx(const FSourceControlPreSubmitFilesValidationExDelegate& PreSubmitFilesValidationDelegate) = 0;

	/**
	 * Unregister a delegate called before submitting changes
	 */
	virtual void UnregisterPreSubmitFilesValidation() = 0;

	/**
	 * Gets currently registered delegates for pre-submit files validation
	 */
	virtual FSourceControlPreSubmitFilesValidationExDelegate GetRegisteredPreSubmitFilesValidationEx() = 0;

	/** 
	 * Register a delegate that is invokes right before files are submitted to source control. @see FSourceControlPreSubmitFinalizeDelegate
	 */
	UE_DEPRECATED(5.1, "ISourceControlModule::RegisterPreSubmitFinalize is deprecated, the functionality will be removed")
	virtual FDelegateHandle RegisterPreSubmitFinalize(const FSourceControlPreSubmitFinalizeDelegate::FDelegate& Delegate) = 0;

	/**
	 * Unregister a previously registered delegate. @see FSourceControlPreSubmitFinalizeDelegate
	 */
	UE_DEPRECATED(5.1, "ISourceControlModule::UnregisterPreSubmitFinalize is deprecated, the functionality will be removed")
	virtual void UnregisterPreSubmitFinalize(FDelegateHandle Handle) = 0;

	/** 
	 * Returns access to the delegate so that it can be broadcast as needed. @see FSourceControlPreSubmitFinalizeDelegate
	 */
	UE_DEPRECATED(5.1, "ISourceControlModule::GetOnPreSubmitFinalize is deprecated, the functionality will be removed")
	virtual const FSourceControlPreSubmitFinalizeDelegate& GetOnPreSubmitFinalize() const = 0;

	/**
	 * Registers a delegate that is invoked after source control operation deleted files.  @see FSourceControlFilesDeletedDelegate
	 */
	virtual FDelegateHandle RegisterFilesDeleted(const FSourceControlFilesDeletedDelegate::FDelegate& InDelegate) = 0;
	
	/**
	 * Unregister a previously registered delegate. @see FSourceControlFilesDeletedDelegate
	 */
	virtual void UnregisterFilesDeleted(FDelegateHandle InHandle) = 0;

	/**
	 * Returns access to the delegate so that it can be broadcast as needed. @see FSourceControlFilesDeletedDelegate
	 */
	virtual const FSourceControlFilesDeletedDelegate& GetOnFilesDeleted() const = 0;

	/**
	 * Register a delegate that returns information on custom projects under source control
	 */
	virtual void RegisterCustomProjectsDelegate(FSourceControlCustomProjectsDelegate InCustomProjectsDelegate) = 0;

	/**
	 * Unregister the FSourceControlCustomProjectsDelegate delegate
	 */
	virtual void UnregisterCustomProjectsDelegate() = 0;

	/**
	 * Return information on custom projects under source control
	 * @note The first entry is considered to be the main project
	 */
	virtual TArray<FSourceControlProjectInfo> GetCustomProjects() const = 0;

	/**
	 * Delegate called whenever the list of projects that GetCustomProjects returns has changed.
	 * @note It is called automatically for RegisterCustomProjectsDelegate and UnregisterCustomProjectsDelegate, but outside of that 
	         it is the responsibility of the code that registered FSourceControlCustomProjectsDelegate to handle change notifications.
	 */
	virtual FSimpleMulticastDelegate& OnCustomProjectsChanged() = 0;

	/**
	 * Return the main custom project under source control or the uproject dir (FPaths::ProjectDir) if there's no custom project
	 */
	virtual FString GetSourceControlProjectDir() const = 0;

	UE_DEPRECATED(5.5, "Use RegisterCustomProjectsDelegate instead.")
	virtual void RegisterSourceControlProjectDirDelegate(const FSourceControlProjectDirDelegate& SourceControlProjectDirDelegate) = 0;

	UE_DEPRECATED(5.5, "Use UnregisterCustomProjectsDelegate instead.")
	virtual void UnregisterSourceControlProjectDirDelegate() = 0;

	UE_DEPRECATED(5.5, "Use !GetCustomProjects().IsEmpty() instead.")
	virtual bool UsesCustomProjectDir() const = 0;

	/**
	 * Returns the object used to monitor the source control status of a collection of files.
	 */
	virtual FSourceControlFileStatusMonitor& GetSourceControlFileStatusMonitor() = 0;

	/**
	 * Gets a reference to the source control module instance.
	 *
	 * @return A reference to the source control module.
	 */
	static inline ISourceControlModule& Get()
	{
		static FName SourceControlModule("SourceControl");
		return FModuleManager::LoadModuleChecked<ISourceControlModule>(SourceControlModule);
	}

	/**
	 * Gets a pointer to the source control module instance, if loaded.
	 *
	 * @return A pointer to the source control module, or null if not loaded.
	 */
	static inline ISourceControlModule* GetPtr()
	{
		static FName SourceControlModule("SourceControl");
		return FModuleManager::GetModulePtr<ISourceControlModule>(SourceControlModule);
	}
};
