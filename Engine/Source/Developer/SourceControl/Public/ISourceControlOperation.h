// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SourceControlResultInfo.h"

class ISourceControlOperation : public TSharedFromThis<ISourceControlOperation, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlOperation() = default;

	/** Get the name of this operation, used as a unique identifier */
	virtual FName GetName() const = 0;

	/** Get the string to display when this operation is in progress */
	virtual FText GetInProgressString() const
	{
		return FText();
	}

	/** Retrieve any info or error messages that may have accumulated during the operation. */
	virtual const FSourceControlResultInfo& GetResultInfo() const
	{
		// Implemented in subclasses
		static const FSourceControlResultInfo ResultInfo = FSourceControlResultInfo();
		return ResultInfo;
	}

	/** Add info/warning message. */
	virtual void AddInfoMessge(const FText& InInfo)
	{
		// Implemented in subclasses
	}

	/** Add error message. */
	virtual void AddErrorMessge(const FText& InError)
	{
		// Implemented in subclasses
	}

	/** Add tag. */
	virtual void AddTag(const FString& InTag)
	{
		// Implemented in subclasses
	}

	/**
	 * Append any info or error messages that may have accumulated during the operation prior
	 * to returning a result, ensuring to keep any already accumulated info.
	 */
	virtual void AppendResultInfo(const FSourceControlResultInfo& InResultInfo)
	{
		// Implemented in subclasses
	}

	/**
	 * This will return true if the operation can be safely called from a background thread.
	 * Currently it is assumed to only the operation 'FDownloadFile' will return true at least
	 * until the API is made thread safe.
	 */
	virtual bool CanBeCalledFromBackgroundThreads() const
	{
		return false;
	}

	/**
	 * Returns true when the operation is allowed to log errors immediately when they are encountered.
	 */
	virtual bool ShouldLogErrors() const
	{
		return true;
	}

	/**
	 * Returns true when the operation is allowed to log info messages immediately when they are encountered.
	 */
	virtual bool ShouldLogInfos() const
	{
		return true;
	}

	/** Factory method for easier operation creation */
	template<typename Type, typename... TArgs>
	static TSharedRef<Type, ESPMode::ThreadSafe> Create(TArgs&&... Args)
	{
		return MakeShareable( new Type(Forward<TArgs>(Args)...));
	}
};

typedef TSharedRef<class ISourceControlOperation, ESPMode::ThreadSafe> FSourceControlOperationRef;
