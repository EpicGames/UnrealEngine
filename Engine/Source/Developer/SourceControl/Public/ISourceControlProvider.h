// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Features/IModularFeature.h"
#include "HAL/Platform.h"
#include "ISourceControlChangelist.h"
#include "ISourceControlChangelistState.h"
#include "ISourceControlOperation.h"
#include "ISourceControlState.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

class FName;
class UPackage;
template <typename FuncType> class TFunctionRef;

#ifndef SOURCE_CONTROL_WITH_SLATE
	#error "SOURCE_CONTROL_WITH_SLATE must be defined. Did you forget a dependency on the 'SourceControl' module?"
#endif

class FSourceControlInitSettings;
class ISourceControlLabel;

/**
 * Hint for how to execute the operation. Note that asynchronous operations require
 * Tick() to be called to manage completed operations.
 */
namespace EConcurrency
{
	enum Type
	{
		/** Force the operation to be issued on the same thread, blocking until complete. */
		Synchronous,
		/** Run the command on another thread, returning immediately. */
		Asynchronous
	};
}

/**
 * Hint to provider when updating state
 */
namespace EStateCacheUsage
{
	enum Type
	{
		/** Force a synchronous update of the state of the file. */
		ForceUpdate,
		/** Use the cached state if possible */
		Use,
	};
}

/**
 * Results of a command execution
 */
namespace ECommandResult
{
	enum Type
	{
		/** Command failed to execute correctly or was not supported by the provider. */
		Failed,
		/** Command executed successfully */
		Succeeded,
		/** Command was canceled before completion */
		Cancelled,
	};
}

/** Delegate used by providers for when operations finish */
DECLARE_DELEGATE_TwoParams(FSourceControlOperationComplete, const FSourceControlOperationRef&, ECommandResult::Type);

/** Delegate used by providers to create source control operations */
DECLARE_DELEGATE_RetVal(FSourceControlOperationRef, FGetSourceControlOperation);

/** Delegate called when the state of an item (or group of items) has changed. */
DECLARE_MULTICAST_DELEGATE(FSourceControlStateChanged);

/**
 * Interface to talking with source control providers.
 */
class ISourceControlProvider : public IModularFeature
{
public:

	/** Key value for common source control provider statuses */
	enum class EStatus
	{
		Enabled,
		Connected,
		Port,
		User,
		Client,
		Repository,
		Remote,
		Branch,
		Email,
		ScmVersion,
		PluginVersion,
		Workspace,
		WorkspacePath,
		Changeset
	};

	/** Flags used when calling ISourceControlProvider::Init */
	enum class EInitFlags : uint32
	{
		/** No flags set */
		None				= 0,
		/** A connection attempt to the provider will be made as part of the initialization */
		AttemptConnection	= 1 << 0,
		/** Errors will not be logged but will still be returned to the caller */
		SuppressErrorLogging = 1 << 1,
		SupressErrorLogging UE_DEPRECATED(5.8, "Spelling typo, please use ISourceControlProvider::EInitFlags::SuppressErrorLogging instead") = 1 << 1
	};

	/** The result info returned form calls to  ISourceControlProvider::Init */
	struct FInitResult
	{
		/** True if the provider is ready for use, otherwise false. This value should match calling ISourceControlProvider::IsAvailable. */
		bool bIsAvailable = false;

		/** 
		 * Contains the important settings used when trying to make a server connection.
		 * This will only contain entries if Init is called with the EInitFlags::AttemptConnection flag.
		 * 
		 * This map should still contain entries if the connection failed.
		 */
		TMap<EStatus, FString> ConnectionSettings;

		/** Contains errors encountered during initialization */
		struct FConnectionErrors
		{
			/** A higher level error message describing at what point in the connection something went wrong. */
			FText ErrorMessage;

			/** A collection of lower level error messages usually from the source revision api itself. */
			TArray<FText> AdditionalErrors;

			FConnectionErrors() = default;
			FConnectionErrors(const FText& InErrorMessage, const TArray<FText>& InAdditionalErrors)
				: ErrorMessage(InErrorMessage), AdditionalErrors(InAdditionalErrors)
			{
			}

			~FConnectionErrors() = default;
		} Errors;
	};

public:
	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlProvider() = default;

	/**
	 * Initialize source control provider.
	 * @param	bForceConnection	If set, this flag forces the provider to attempt a connection to its server.
	 */
	virtual void Init(bool bForceConnection = true) = 0;

	/**
	 * Initialize source control provider.
	 * @param Flags		Used to specialize initialization behavior, @see EInitFlags
	 * 
	 * @return			A structure containing the result as well as any errors. @see FInitResult
	 */
	SOURCECONTROL_API virtual FInitResult Init(EInitFlags Flags);

	/**
	 * Shut down source control provider.
	 */
	virtual void Close() = 0;

	/** Get the source control provider name */
	virtual const FName& GetName() const = 0;

	/** Get the source control status as plain, human-readable text */
	virtual FText GetStatusText() const = 0;

	/** Get the source control status as a series of key value pairs */
	virtual TMap<EStatus, FString> GetStatus() const = 0;

	/** Quick check if source control is enabled. Specifically, it returns true if a source control provider is set (regardless of whether the provider is available) and false if no provider is set. So all providers except the stub defaultSourceProvider will return true. */
	virtual bool IsEnabled() const = 0;

	/**
	 * Quick check if source control is available for use (server-based providers can use this
	 * to return whether the server is available or not)
	 *
	 * @return	true if source control is available, false if it is not
	 */
	virtual bool IsAvailable() const = 0;

	/**
	 * Login to the source control server (if any).
	 * This is just a wrapper around Execute().
	 * @param	InPassword						The password (if required) to use when logging in.
	 * @param	InConcurrency					How to execute the operation, blocking or asynchronously on another thread.
	 * @param	InOperationCompleteDelegate		Delegate to call when the operation is completed. This is called back internal to this call when executed on the main thread, or from Tick() when queued for asynchronous execution. If the provider is not enabled or if the command is not suported the delegate is immediately called with ECommandResult::Failed.
	 * @return the result of the operation.
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Login( const FString& InPassword = FString(), EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	* Queries branch configuration from source control
	* @param	ConfigSrc			The source path to the branch configuration file in source control
	* @param	ConfigDest			The destination path to save the configuration to for parsing
	*/
	virtual bool QueryStateBranchConfig(const FString& ConfigSrc, const FString& ConfigDest) = 0;

	/**
	* Register branches to query for state in addition to the current branch
	* @param	BranchNames			Names of the branches to query
	* @param	ContentRoot			Path to the content root for branch mapping
	*/
	virtual void RegisterStateBranches(const TArray<FString>& BranchNames, const FString& ContentRoot) = 0;

	/**
	* Register branches to query for state in addition to the current branch
	* @param	BranchNames			Names of the branches to query
	* @param	ContentRoots			Paths to the content roots for branch mapping
	*/
	SOURCECONTROL_API virtual void RegisterStateBranches( const TArray<FString>& BranchNames, const TArray<FString>& ContentRoots );

	/**
	* Gets the state index of the specified branch, higher index branches are generally closer to releases
	* @param	BranchName			Name of the branch to query
	* @return	the index of the specified branch
	*/
	virtual int32 GetStateBranchIndex(const FString& BranchName) const = 0;

	/**
	* Gets the name of the state branch at the given index, higher index branches are generally closer to releases
	* @param	BranchIndex			Index of the branch to retrieve
	* @param	OutBrancheName[out] Name of the branch, if successful
	* @return true if the branch name could be retrieved, false otherwise.
	*/
	virtual bool GetStateBranchAtIndex(int32 BranchIndex, FString& OutBranchName) const = 0;

	/**
	 * Get the state of each of the passed-in files. State may be cached for faster queries. Note states can be NULL!
	 * @param	InFiles				The files to retrieve state for.
	 * @param	OutState			The states of the files. This will be empty if the operation fails. Note states can be NULL!
	 * @param	InStateCacheUsage	Whether to use the state cache or to force a (synchronous) state retrieval.
	 * @return the result of the operation.
	 */
	virtual ECommandResult::Type GetState( const TArray<FString>& InFiles, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage ) = 0;

	/**
	 * Helper overload for state retrieval, see GetState().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type GetState( const TArray<UPackage*>& InPackages, TArray<FSourceControlStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage );

	/**
	 * Helper overload for state retrieval, see GetState().
	 */
	SOURCECONTROL_API virtual FSourceControlStatePtr GetState( const FString& InFile, EStateCacheUsage::Type InStateCacheUsage );

	/**
	 * Helper overload for state retrieval, see GetState().
	 */
	SOURCECONTROL_API virtual FSourceControlStatePtr GetState( const UPackage* InPackage, EStateCacheUsage::Type InStateCacheUsage );

	/**
	 * Get the state of each of the passed-in changelists. State may be cached for faster queries. Note states can be NULL!
	 * @param	InChangelists		The changelists to retrieve state for.
	 * @param	OutState			The states of the changelists. This will be empty if the operation fails. Note states can be NULL!
	 * @param	InStateCacheUsage	Whether to use the state cache or to force a (synchronous) state retrieval.
	 * @return the result of the operation.
	 */
	virtual ECommandResult::Type GetState( const TArray<FSourceControlChangelistRef>& InChangelists, TArray<FSourceControlChangelistStateRef>& OutState, EStateCacheUsage::Type InStateCacheUsage ) = 0;

	/**
	 * Helper overload for state retrieval, see GetState().
	 */
	SOURCECONTROL_API virtual FSourceControlChangelistStatePtr GetState( const FSourceControlChangelistRef& InChangelist, EStateCacheUsage::Type InStateCacheUsage );

	/**
	 * Get all cached source control state objects for which the supplied predicate returns true
	 */
	virtual TArray<FSourceControlStateRef> GetCachedStateByPredicate(TFunctionRef<bool(const FSourceControlStateRef&)> Predicate) const = 0;

	/**
	 * Register a delegate to be called when source control state(s) change
	 */
	virtual FDelegateHandle RegisterSourceControlStateChanged_Handle( const FSourceControlStateChanged::FDelegate& SourceControlStateChanged ) = 0;

	/**
	 * Unregister a delegate to be called when source control state(s) change
	 */
	virtual void UnregisterSourceControlStateChanged_Handle( FDelegateHandle Handle ) = 0;

	/**
	 * Attempt to execute an operation on the passed-in files (if any are required).
	 * @param	InOperation						The operation to perform.
	 * @param	InChangelist                    The changelist to operate on, can be null.
	 * @param	InFiles							The files to operate on.
	 * @param	InConcurrency					How to execute the operation, blocking or asynchronously on another thread.
	 * @param	InOperationCompleteDelegate		Delegate to call when the operation is completed. This is called back internal to this call when executed on the main thread, or from Tick() when queued for asynchronous execution. If the provider is not enabled or if the command is not suported the delegate is immediately called with ECommandResult::Failed.
	 * @return the result of the operation.
	 */
	virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() ) = 0;

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Execute(const FSourceControlOperationRef& InOperation, const TArray<FString>& InFiles, EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete());

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, const EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, const UPackage* InPackage, const EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, const FString& InFile, const EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * Helper overload for operation execution, see Execute().
	 */
	SOURCECONTROL_API virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, const TArray<UPackage*>& InPackages, const EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete() );

	/**
	 * Helper overload for operation execution, see Execute().
	*/
	SOURCECONTROL_API virtual ECommandResult::Type Execute( const FSourceControlOperationRef& InOperation, FSourceControlChangelistPtr InChangelist, const EConcurrency::Type InConcurrency = EConcurrency::Synchronous, const FSourceControlOperationComplete& InOperationCompleteDelegate = FSourceControlOperationComplete());

	/**
	 * Check to see if we can execute an operation.
	 * @param	InOperation		The operation to check.
	 * @return true if the operation can be executed.
	 */
	virtual bool CanExecuteOperation( const FSourceControlOperationRef& InOperation ) const = 0;

	/**
	 * Check to see if we can cancel an operation.
	 * @param	InOperation		The operation to check.
	 * @return true if the operation was cancelled.
	 */
	virtual bool CanCancelOperation( const FSourceControlOperationRef& InOperation ) const = 0;

	/**
	 * Attempt to cancel an operation in progress.
	 * @param	InOperation		The operation to attempt to cancel.
	 */
	virtual void CancelOperation( const FSourceControlOperationRef& InOperation ) = 0;

	/**
	 * Get a label matching the passed-in name.
	 * @param	InLabelName	String specifying the label name
	 * @return the label, if any
	 */
	SOURCECONTROL_API virtual TSharedPtr<class ISourceControlLabel> GetLabel( const FString& InLabelName ) const;

	/**
	 * Get an array of labels matching the passed-in spec.
	 * @param	InMatchingSpec	String specifying the label spec.
	 * @return an array of labels matching the spec.
	 */
	virtual TArray< TSharedRef<class ISourceControlLabel> > GetLabels( const FString& InMatchingSpec ) const = 0;

	/**
	 * Returns the list of available changelists if the underlying source control supports the 'changelist' concept.
	 *
	 * @param InStateCacheUsage True to retrieve the list from a local cache, false to request it from the server (if any).
	 */
	virtual TArray<FSourceControlChangelistRef> GetChangelists(EStateCacheUsage::Type InStateCacheUsage) = 0;

	/**
	 * Executes the FDownloadFile operation, but unlike the ::Execute method this can be called from a background thread, this
	 * works because FDownloadFile is thread safe and it does not change the state of source control.
	 * NOTE: This is only intended for use by the virtualization module and will be deprecated at some point in the future when
	 * thread safety is built into the system.
	 * NOTE: This is only implemented for the perforce source control provider with no plans to extend this to other providers.
	 * 
	 * @param InOperation The download operation to be executed.
	 * @param InFile The depot path of a file to download.
	 * @return True if the operation was a success, otherwise false.
	 */
	SOURCECONTROL_API bool TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const FString& InFile);

	/**
	 * Executes the FDownloadFile operation, but unlike the ::Execute method this can be called from a background thread, this
	 * works because FDownloadFile is thread safe and it does not change the state of source control.
	 * NOTE: This is only intended for use by the virtualization module and will be deprecated at some point in the future when
	 * thread safety is built into the system.
	 * NOTE: This is only implemented for the perforce source control provider with no plans to extend this to other providers.
	 *
	 * @param InOperation The download operation to be executed.
	 * @param InFile An array of depot paths for the files to download.
	 * @return True if the operation was a success, otherwise false.
	 */
	SOURCECONTROL_API virtual bool TryToDownloadFileFromBackgroundThread(const TSharedRef<class FDownloadFile>& InOperation, const TArray<FString>& InFiles);

	/**
	 * Used to switch the provider from one workspace to another. 
	 * NOTE: This concept is currently only implemented for the perforce source control provider.
	 * 
	 * @param NewWorkspaceName			The name of the workspace to switch to
	 * @param ResultInfo[out]			Errors and info messages generated will be written here
	 * @param OutOldWorkspaceName[out]	The name of the previous workspace will be written to this FString if the pointer is valid (optional)
	 * 
	 * @return The result of the operation.
	 */
	SOURCECONTROL_API virtual ECommandResult::Type SwitchWorkspace(FStringView NewWorkspaceName, FSourceControlResultInfo& OutResultInfo, FString* OutOldWorkspaceName);

	/**
	 * Whether the provider uses local read-only state to signal whether a file is editable.
	 */
	virtual bool UsesLocalReadOnlyState() const = 0;

	/**
	 * Whether the provider uses changelists to identify commits/revisions
	 */
	virtual bool UsesChangelists() const = 0;

	/**
	 * Whether the provider supports uncontrolled changelists to allow work offline
	 */
	virtual bool UsesUncontrolledChangelists() const = 0;

	/**
	 * Whether the provider uses the checkout workflow
	 */
	virtual bool UsesCheckout() const = 0;

	/**
	 * Whether the provider uses individual file revisions
	 */
	virtual bool UsesFileRevisions() const = 0;

	/**
	 * Whether the provider uses snapshots
	 */
	virtual bool UsesSnapshots() const = 0;

	/**
	 * Whether the provider uses soft revert before delete
	 */
	virtual bool UsesSoftRevertOnDelete() const = 0;

	/**
	 * Whether the provider allow a diff between a changed file and the depot
	 */
	virtual bool AllowsDiffAgainstDepot() const = 0;

	/**
	 * Whether there is anything to sync
	 */
	virtual TOptional<bool> HasChangesToSync() const = 0;

	/**
	 * Whether there is anything to check-in
	 */
	virtual TOptional<bool> HasChangesToCheckIn() const = 0;

	UE_DEPRECATED(5.8, "This function is no longer used.")
	virtual TOptional<bool> IsAtLatestRevision() const final { return TOptional<bool>(); }

	UE_DEPRECATED(5.8, "This function is no longer used.")
	virtual TOptional<int> GetNumLocalChanges() const final { return TOptional<int>(); }

	/**
	 * Called every update.
	 */
	virtual void Tick() = 0;

#if SOURCE_CONTROL_WITH_SLATE
	/**
	 * Create a settings widget for display in the login window.
	 * @returns a widget used to edit the providers settings required prior to connection.
	 */
	virtual TSharedRef<class SWidget> MakeSettingsWidget() const = 0;
#endif // SOURCE_CONTROL_WITH_SLATE

protected:

	/* 
	 * The ::Create method is an easy way for us to create new providers, but we really don't 
	 * want anything except FSourceControlModule calling it. For now we achieve this by having
	 * ::Create protected and making FSourceControlModule a friend. 
	 * TODO Move the create to a factory system via IModularFeatures and remove the friend declaration.
	 */
	friend class FSourceControlModule;

	/** 
	 * Creates a new instance of the source control provider type. Derived providers
	 * should implement this if they can more than one provider of the same type existing. 
	 * @see ISourceControlModule::CreateProvider
	 */
	virtual TUniquePtr<ISourceControlProvider> Create(const FStringView& OwnerName, const FSourceControlInitSettings& InitialSettings) const { return TUniquePtr<ISourceControlProvider>(); }
};

ENUM_CLASS_FLAGS(ISourceControlProvider::EInitFlags);
