// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ISourceControlProvider.h"

/**
 * A single line of an annotated file
 */
class FAnnotationLine
{
public:
	FAnnotationLine(int32 InChangeNumber, const FString& InUserName, const FString& InLine)
		: ChangeNumber(InChangeNumber)
		, UserName(InUserName)
		, Line(InLine)
	{
	}

	int32 ChangeNumber;
	FString UserName;
	FString Line;
};

/** 
 * Abstraction of a source control revision.
 */
class ISourceControlRevision : public TSharedFromThis<ISourceControlRevision, ESPMode::ThreadSafe>
{
public:
	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlRevision() = default;

	/** 
	 * Get this revision of the file & store it in a temp file.
	 * @param	InOutFilename		The filename that the revision will be written to. If this is empty a temp filename will be generated and returned in this string.
	 * @param	InConcurrency		The concurrency type of the get operation. Should be used when the revision is not requested from the main thread.
	 * @return true if the operation succeeded.
	 */
	virtual bool Get(FString& InOutFilename, EConcurrency::Type InConcurrency = EConcurrency::Synchronous) const = 0;

	/** 
	 * Get an annotated revision of the file & store it in a temp file.
	 * @param	OutLines		Array of lines representing the contents of the file.
	 * @return true if the operation succeeded.
	 */
	virtual bool GetAnnotated( TArray<FAnnotationLine>& OutLines ) const = 0;

	/** 
	 * Get an annotated revision of the file & store it in a temp file.
	 * @param	InOutFilename		The filename that the revision will be written to. If this is empty a temp filename will be generated and returned in this string.
	 * @return true if the operation succeeded.
	 */
	virtual bool GetAnnotated( FString& InOutFilename ) const = 0;

	/** Get the local filename of this file. */
	virtual const FString& GetFilename() const = 0;

	/** Number of the revision */
	virtual int32 GetRevisionNumber() const = 0;

	/** String representation of the revision */
	virtual const FString& GetRevision() const = 0;

	/** Changelist/Commit description */
	virtual const FString& GetDescription() const = 0;

	/** User name of the submitter */
	virtual const FString& GetUserName() const = 0;

	/** Workspace/Clientspec of the submitter (if any) */
	virtual const FString& GetClientSpec() const = 0;

	/** Action taken to the file this revision (branch/integrate/edit/etc.) */
	virtual const FString& GetAction() const = 0;

	/** Source of branch, if any */
	virtual TSharedPtr<ISourceControlRevision, ESPMode::ThreadSafe> GetBranchSource() const = 0;

	/** Date of the revision */
	virtual const FDateTime& GetDate() const = 0;

	/** Changelist number/revision number of the revision - an identifier for the check-in */
	virtual int32 GetCheckInIdentifier() const = 0;

	/** File size of the revision (0 if the file was deleted) */
	virtual int32 GetFileSize() const = 0;
};
