// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if SOURCE_CONTROL_WITH_SLATE
#include "Textures/SlateIcon.h"
#endif //SOURCE_CONTROL_WITH_SLATE

class ISourceControlRevision;
class ISourceControlState;

typedef TSharedRef<class ISourceControlState, ESPMode::ThreadSafe> FSourceControlStateRef;
typedef TSharedPtr<class ISourceControlState, ESPMode::ThreadSafe> FSourceControlStatePtr;

class ISourceControlChangelist;

typedef TSharedRef<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistRef;
typedef TSharedPtr<class ISourceControlChangelist, ESPMode::ThreadSafe> FSourceControlChangelistPtr;

/**
 * An abstraction of the state of a file under source control
 */
class ISourceControlState : public TSharedFromThis<ISourceControlState, ESPMode::ThreadSafe>
{
public:
	enum { INVALID_REVISION = -1 };

	struct FResolveInfo
	{
		FString RemoteFile;
		FString BaseFile;
		FString RemoteRevision;
		FString BaseRevision;

		bool IsValid() const
		{
			return !RemoteRevision.IsEmpty() && !RemoteFile.IsEmpty();
		}

		operator bool() const
		{
			return IsValid();
		}
	};

	/**
	 * Virtual destructor
	 */
	virtual ~ISourceControlState() = default;

	/** 
	 * Get the size of the history. 
	 * If an FUpdateStatus operation has been called with the ShouldUpdateHistory() set, there 
	 * should be history present if the file has been committed to source control.
	 * @returns the number of items in the history
	 */
	virtual int32 GetHistorySize() const = 0;

	/**
	 * Get an item from the history
	 * @param	HistoryIndex	the index of the history item
	 * @returns a history item or NULL if none exist
	 */
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetHistoryItem( int32 HistoryIndex ) const = 0;

	/**
	 * Find an item from the history with the specified revision number.
	 * @param	RevisionNumber	the revision number to look for
	 * @returns a history item or NULL if the item could not be found
	 */
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision( int32 RevisionNumber ) const = 0;
	
	/**
	 * Find an item from the history with the specified revision.
	 * @param	InRevision	the revision identifier to look for
	 * @returns a history item or NULL if the item could not be found
	 */
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> FindHistoryRevision( const FString& InRevision ) const = 0;

	/**
	 * Get the revision that we should use as a base when performing a three way merge, does not refresh source control state
	 * @returns a revision identifier or NULL if none exist
	 */
	UE_DEPRECATED(5.3, "Use GetResolveInfo() and FindHistoryRevision() instead")
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetBaseRevForMerge() const final { return FindHistoryRevision(GetResolveInfo().BaseRevision); }

	/**
	 * Get the file and revision number of the base and remote assets considered in a merge resolve
	 * @returns a valid FResolveInfo if the asset is being resolved, otherwise FResolveInfo::IsValid() will return false
	 */
	virtual FResolveInfo GetResolveInfo() const {return {};}

	/**
	 * Get the revision that we are currently synced to
	 * @returns a revision identifier or NULL if none exist
	 */
	virtual TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> GetCurrentRevision() const = 0;

#if SOURCE_CONTROL_WITH_SLATE
	/**
	 * Gets the icon we should use to display the state in a UI.
	 */
	virtual FSlateIcon GetIcon() const = 0;
#endif // SOURCE_CONTROL_WITH_SLATE

	/**
	 * Get the name of the icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */
	UE_DEPRECATED(5.0, "GetIconName has been replaced by GetIcon.")
	virtual FName GetIconName() const { return NAME_None; }
		
	/**
	 * Get the name of the small icon graphic we should use to display the state in a UI.
	 * @returns the name of the icon to display
	 */	
	UE_DEPRECATED(5.0, "GetSmallIconName has been replaced by GetIcon.")
	virtual FName GetSmallIconName() const { return NAME_None; }

	/**
	 * Get a text representation of the state
	 * @returns	the text to display for this state
	 */
	virtual FText GetDisplayName() const = 0;

	/**
	 * Get a tooltip to describe this state
	 * @returns	the text to display for this states tooltip
	 */
	virtual FText GetDisplayTooltip() const = 0;

	/**
	 * Get the local filename that this state represents
	 * @returns	the filename
	 */
	virtual const FString& GetFilename() const = 0;

	/**
	 * Get the timestamp of the last update that was made to this state.
	 * @returns	the timestamp of the last update
	 */
	virtual const FDateTime& GetTimeStamp() const = 0;

	/** Get whether this file can be checked in. */
	virtual bool CanCheckIn() const = 0;

	/** Get whether this file can be checked out */
	virtual bool CanCheckout() const = 0;

	/** Get whether this file is checked out by the current user*/
	virtual bool IsCheckedOut() const = 0;

	/** Get whether this file is checked out by someone else in the current branch */
	virtual bool IsCheckedOutOther(FString* Who = NULL) const = 0;

	/** Get whether this file is checked out in a different branch, if no branch is specified defaults to FEngineVerion current branch */
	virtual bool IsCheckedOutInOtherBranch(const FString& CurrentBranch = FString()) const = 0;

	/** Get whether this file is modified in a different branch, if no branch is specified defaults to FEngineVerion current branch */
	virtual bool IsModifiedInOtherBranch(const FString& CurrentBranch = FString()) const = 0;

	/** Get whether this file is checked out or modified in a different branch, if no branch is specified defaults to FEngineVerion current branch */
	virtual bool IsCheckedOutOrModifiedInOtherBranch(const FString& CurrentBranch = FString()) const = 0;

	/** Get the other branches this file is checked out in */
	virtual TArray<FString> GetCheckedOutBranches() const = 0;

	/** Get the user info for checkouts on other branches */
	virtual FString GetOtherUserBranchCheckedOuts() const = 0;

	/** Get head modification information for other branches
	 * @returns true with parameters populated if there is a branch with a newer modification (edit/delete/etc)  
	*/
	virtual bool GetOtherBranchHeadModification(FString& HeadBranchOut, FString& ActionOut, int32& HeadChangeListOut) const = 0;

	virtual FSourceControlChangelistPtr GetCheckInIdentifier() const;

	/** Get whether this file is up-to-date with the version in source control */
	virtual bool IsCurrent() const = 0;

	/** Get whether this file is under source control */
	virtual bool IsSourceControlled() const = 0;

	/** Get whether this file is local to the current user (ie, has never been pushed to anyone else) */
	virtual bool IsLocal() const
	{
		return IsAdded() || !IsSourceControlled() || IsIgnored();
	}

	/** Get whether this file is marked for add */
	virtual bool IsAdded() const = 0;

	/** Get whether this file is marked for delete */
	virtual bool IsDeleted() const = 0;

	/** Get whether this file is ignored by source control */
	virtual bool IsIgnored() const = 0;

	/** Get whether source control allows this file to be edited */
	virtual bool CanEdit() const = 0;

	/** Get whether source control allows this file to be deleted. */
	virtual bool CanDelete() const = 0;

	/** Get whether we know anything about this files source control state */
	virtual bool IsUnknown() const = 0;

	/** Get whether this file is modified compared to the version we have from source control */
	virtual bool IsModified() const = 0;

	/** 
	 * Get whether this file can be added to source control (i.e. is part of the directory 
	 * structure currently under source control) 
	 */
	virtual bool CanAdd() const = 0;

	/** Get whether this file is in a conflicted state */
	virtual bool IsConflicted() const
	{
		return GetResolveInfo().IsValid();
	}

	/** Get whether this file can be reverted, i.e. its changes are discarded and the file will no longer be checked-out. */
	virtual bool CanRevert() const = 0;

	/** Gets the warnings messages associated with this state, if any. Ex 'checkout by other', 'out of date', etc). */
	virtual TOptional<FText> GetWarningText() const
	{ 
		TOptional<FText> WarningText;
		if (IsConflicted() ||
			!IsCurrent() ||
			IsCheckedOutOther() ||
			(!IsCheckedOut() && (IsCheckedOutInOtherBranch() || IsModifiedInOtherBranch())))
		{
			WarningText.Emplace(GetDisplayTooltip()); // The tooltip text usually describe well enough the warning.
		}
		return WarningText;
	}

	/** Gets the status message associated with this state, if any. This is a superset of the GetWarningText that also includes IsCheckedOut. */
	virtual TOptional<FText> GetStatusText() const
	{
		TOptional<FText> StatusText = GetWarningText();
		if (!StatusText.IsSet() && IsCheckedOut())
		{
			StatusText.Emplace(GetDisplayTooltip());
		}
		return StatusText;
	}
};

inline FSourceControlChangelistPtr ISourceControlState::GetCheckInIdentifier() const
{
	return {};
}

