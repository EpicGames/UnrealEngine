// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if SOURCE_CONTROL_WITH_SLATE

#include "Styling/SlateStyle.h"

/**
 * The style manager that is used to access the currently active revision control style.
 * Use FRevisionControlStyleManager::Get() to access and use any revision control icons/styles
 */
class FRevisionControlStyleManager
{
public:

	~FRevisionControlStyleManager();

	/** reloads textures used by slate renderer */
	static SOURCECONTROL_API void ReloadTextures();

	/** @return The current revision control style being used */
	static SOURCECONTROL_API const ISlateStyle& Get();

	/** @return The name of the current revision control style being used */
	static SOURCECONTROL_API FName GetStyleSetName();

	/** Set the active revision control style to the input style name */
	static SOURCECONTROL_API void SetActiveRevisionControlStyle(FName InNewActiveRevisionControlStyleName);

	/** Set the active revision control style to the default style */
	static SOURCECONTROL_API void ResetToDefaultRevisionControlStyle();

private:

	// The default revision control style instance
	static TSharedPtr< class ISlateStyle > DefaultRevisionControlStyleInstance;

	// The currently active revision control style
	static FName CurrentRevisionControlStyleName;
};

/**
 * The default revision control style the editor ships with. Inherit from this to create a custom revision controls style
 * Use FRevisionControlStyleManager::SetActiveRevisionControlStyle to change the currently active revision control style
 * Edit the defaults in the constructor here to change any revision control icons in the editor
 */
class FDefaultRevisionControlStyle : public FSlateStyleSet
{
public:
	
	SOURCECONTROL_API FDefaultRevisionControlStyle();
	SOURCECONTROL_API virtual ~FDefaultRevisionControlStyle() override;
	
	virtual const FName& GetStyleSetName() const override;

protected:

	/** The specific color we use for all the "Branched" icons */
	FSlateColor BranchedColor;

	/** The specific colors we use for all the "Status" icons */
	FSlateColor StatusCheckedOutColor;
	FSlateColor StatusCheckedOutByOtherUserColor;
	FSlateColor StatusNotAtHeadRevisionColor;

	/** The specific colors we use for all the "Snapshot History" state icons */
	FSlateColor SnapshotHistoryAdded;
	FSlateColor SnapshotHistoryModified;
	FSlateColor SnapshotHistoryRemoved;
	
private:
	static FName StyleName;
};

#endif //SOURCE_CONTROL_WITH_SLATE
