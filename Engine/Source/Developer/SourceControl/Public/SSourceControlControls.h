// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if SOURCE_CONTROL_WITH_SLATE

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"
#include "Delegates/IDelegateInstance.h"
#include "Framework/SlateDelegates.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

DECLARE_DELEGATE_RetVal(int32, FNumConflicts);

DECLARE_DELEGATE_RetVal(bool, FIsVisible);
DECLARE_DELEGATE_RetVal(bool, FIsEnabled);

/** Widget for displaying Source Control Check in Changes and Sync Latest buttons */
class SSourceControlControls : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SSourceControlControls) {}
		SLATE_ATTRIBUTE(bool, IsEnabledMiddleSeparator)
		SLATE_ATTRIBUTE(bool, IsEnabledRightSeparator)
		SLATE_EVENT(FOnGetContent, OnGenerateKebabMenu)
	SLATE_END_ARGS()

public:
	/** Construct this widget */
	SOURCECONTROL_API void Construct(const FArguments& InArgs);

public:
	/** Separators */
	SOURCECONTROL_API EVisibility GetSourceControlMiddleSeparatorVisibility() const;
	SOURCECONTROL_API EVisibility GetSourceControlRightSeparatorVisibility() const;

	/** Sync button */
	static SOURCECONTROL_API bool IsSourceControlSyncEnabled();
	static SOURCECONTROL_API bool HasSourceControlChangesToSync();
	static SOURCECONTROL_API EVisibility GetSourceControlSyncStatusVisibility();
	static SOURCECONTROL_API FText GetSourceControlSyncStatusText();
	static SOURCECONTROL_API FText GetSourceControlSyncStatusToolTipText();
	static SOURCECONTROL_API const FSlateBrush* GetSourceControlSyncStatusIcon();
	static SOURCECONTROL_API FReply OnSourceControlSyncClicked();

	/** Check-in button */
	static SOURCECONTROL_API bool IsSourceControlCheckInEnabled();
	static SOURCECONTROL_API bool HasSourceControlChangesToCheckIn();
	static SOURCECONTROL_API EVisibility GetSourceControlCheckInStatusVisibility();
	static SOURCECONTROL_API FText GetSourceControlCheckInStatusText();
	static SOURCECONTROL_API FText GetSourceControlCheckInStatusToolTipText();
	static SOURCECONTROL_API const FSlateBrush* GetSourceControlCheckInStatusIcon();
	static SOURCECONTROL_API FReply OnSourceControlCheckInChangesClicked();

	/** Restore as latest button */
	static SOURCECONTROL_API bool IsSourceControlRestoreAsLatestEnabled();
	static SOURCECONTROL_API EVisibility GetSourceControlRestoreAsLatestVisibility();
	static SOURCECONTROL_API FText GetSourceControlRestoreAsLatestText();
	static SOURCECONTROL_API FText GetSourceControlRestoreAsLatestToolTipText();
	static SOURCECONTROL_API const FSlateBrush* GetSourceControlRestoreAsLatestStatusIcon();
	static SOURCECONTROL_API FReply OnSourceControlRestoreAsLatestClicked();

public:
	static SOURCECONTROL_API int32 GetNumConflictsRemaining();
	static SOURCECONTROL_API int32 GetNumConflictsUpcoming();

public:
	static void SetNumConflictsRemaining(const FNumConflicts& InNumConflictsRemaining) { NumConflictsRemaining = InNumConflictsRemaining; }
	static void SetNumConflictsUpcoming(const FNumConflicts& InNumConflictsUpcoming) { NumConflictsUpcoming = InNumConflictsUpcoming; }

	static void SetIsSyncLatestEnabled(const FIsEnabled& InSyncLatestEnabled) { IsSyncLatestEnabled = InSyncLatestEnabled; }
	static void SetIsCheckInChangesEnabled(const FIsEnabled& InCheckInChangesEnabled) { IsCheckInChangesEnabled = InCheckInChangesEnabled; }
	static void SetIsRestoreAsLatestEnabled(const FIsEnabled& InRestoreAsLatestEnabled) { IsRestoreAsLatestEnabled = InRestoreAsLatestEnabled; }

	static void SetIsSyncLatestVisible(const FIsVisible& InSyncLatestVisible) { IsSyncLatestVisible = InSyncLatestVisible; }
	static void SetIsCheckInChangesVisible(const FIsVisible& InCheckInChangesVisible) { IsCheckInChangesVisible = InCheckInChangesVisible; }
	static void SetIsRestoreAsLatestVisible(const FIsVisible& InRestoreAsLatestVisible) { IsRestoreAsLatestVisible = InRestoreAsLatestVisible; }

	static void SetOnSyncLatestClicked(const FOnClicked& InSyncLatestClicked) { OnSyncLatestClicked = InSyncLatestClicked; }
	static void SetOnCheckInChangesClicked(const FOnClicked& InCheckInChangesClicked) { OnCheckInChangesClicked = InCheckInChangesClicked; }
	static void SetOnRestoreAsLatestClicked(const FOnClicked& InRestoreAsLatestClicked) { OnRestoreAsLatestClicked = InRestoreAsLatestClicked; }

private:
	
	TAttribute<bool> IsMiddleSeparatorEnabled;
	TAttribute<bool> IsRightSeparatorEnabled;

	static SOURCECONTROL_API FNumConflicts NumConflictsRemaining;
	static SOURCECONTROL_API FNumConflicts NumConflictsUpcoming;

	static SOURCECONTROL_API FIsEnabled IsSyncLatestEnabled;
	static SOURCECONTROL_API FIsEnabled IsCheckInChangesEnabled;
	static SOURCECONTROL_API FIsEnabled IsRestoreAsLatestEnabled;

	static SOURCECONTROL_API FIsVisible IsSyncLatestVisible;
	static SOURCECONTROL_API FIsVisible IsCheckInChangesVisible;
	static SOURCECONTROL_API FIsVisible IsRestoreAsLatestVisible;

	static SOURCECONTROL_API FOnClicked OnSyncLatestClicked;
	static SOURCECONTROL_API FOnClicked OnCheckInChangesClicked;
	static SOURCECONTROL_API FOnClicked OnRestoreAsLatestClicked;
};

#endif // SOURCE_CONTROL_WITH_SLATE
