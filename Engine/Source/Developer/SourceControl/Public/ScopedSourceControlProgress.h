// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWindow;

class FScopedSourceControlProgress
{
public:
	/** 
	 * Constructor.
	 * Progress dialog is shown immediately (if possible) and is closed when this
	 * object goes out of scope.
	 * @param	InText			The text to display in the progress dialog. If this is empty the dialog will not be shown.
	 * @param	InOnCancelled	The delegate to call when the cancel button is clicked.
	 */
	SOURCECONTROL_API FScopedSourceControlProgress(const FText& InText, const FSimpleDelegate& InOnCancelled = FSimpleDelegate());

	/** Destructor - closes the dialog */
	SOURCECONTROL_API ~FScopedSourceControlProgress();

	/** Makes sure the dialog is ticked */
	SOURCECONTROL_API void Tick();

private:

	/** The window containing this progress window */
	TWeakPtr<class SWindow> WindowPtr;
};
