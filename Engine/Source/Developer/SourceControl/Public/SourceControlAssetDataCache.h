// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Templates/SharedPointer.h"

#include <atomic>

struct FAssetData;

typedef TSharedPtr<TArray<FAssetData>> FAssetDataArrayPtr;

struct FSourceControlAssetDataEntry
{
	FAssetDataArrayPtr	AssetDataArrayPtr;
	TFuture<void>		FetchAssetDataTask;
	bool				bInitialized;
};

struct FAssetDataToLoad
{
	FString TempFilename;
	FString Filename;
};

class FSourceControlAssetDataCache
{
	typedef TMap<FString, FSourceControlAssetDataEntry> FAssetDataCache;

public:
	/** Called when SourceControl module is starting up. */
	void Startup();

	/** Called when SourceControl module is shutting down. */
	void Shutdown();

	/**
	 * Attempt to get AssetData for the provided SourceControlled file.
	 * @param	InFileState				The asset to retrieve AssetData for.
	 * @param	OutAssetDataArrayPtr	If retrieved, the AssetData will be returned in this argument.
	 * @return True if processing is complete for the requested asset. False if the process is incomplete and AssetData could still be updated in the future.
	 */
	bool SOURCECONTROL_API GetAssetDataArray(FSourceControlStateRef InFileState, FAssetDataArrayPtr& OutAssetDataArrayPtr);

	/**
	 * Called every update.
	 */
	void Tick();

	/** Called when Source Control Dialog is shown. Prevents tasks to run while the dialog is opened. */
	void OnSourceControlDialogShown();

	/** Called when Source Control Dialog is closed. Allow work to be resumed. */
	void OnSourceControlDialogClosed();

private:
	/**
	 * Adds a new entry to the AssetData information cache. Retrieving possible informations from SourceControl.
	 * AssetData retrieval process is asynchronous, meaning it could not be available right after this call.
	 * @param	InFileState				The asset to retrieve AssetData for.
	 * @return A pointer to the newly created entry.
	 */
	FSourceControlAssetDataEntry* AddAssetInformationEntry(FSourceControlStateRef InFileState);

	/** Executes the FDownloadFile fast path for files in FilesToDownload. Game thread only. */
	void DispatchDownloadFile();

	/**
	 * Called when the asynchronous FDownloadFile operation launched by DispatchDownloadFile completes.
	 * Runs on the game thread.
	 */
	void OnDownloadFileComplete(const TArray<FString>& InDispatchedFiles, const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);

	/** Executes Source Control Operation to retrieve History for files in FileHistoryToUpdate */
	void DispatchUpdateHistory();

	/**
	 * Called when the asynchronous Source Control Operation launched by GetFileHistory completes.
	 * @param	InUpdatedFiles	An array representing the updated files.
	 * @param	Operation		The Source Control operation used.
	 * @param	InResult		The result of the operation.
	 */
	void OnUpdateHistoryComplete(const TArray<FString>& InUpdatedFiles, const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
	
	/** Launches AssetData retrieval tasks for files contained in AssetDataToFetch */
	void DispatchFetchAssetData();

	/** The payload executed by AssetData retrieval tasks */
	void FetchAssetData(FSourceControlStateRef InFileState);

	/** Called after a FetchAssetData retrieval task has been executed */
	void OnFetchAssetDataComplete();
	
	/**  Updates the AssetData information cache with the information retrieved from Source Control */
	void UpdatePendingAssetData();

	/** Callback called when the SourceControlProvider is changed */
	void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

	/** Wait for the currently running tasks to finish and clears all queues. */
	void ClearPendingTasks();

	/** Clears the asset data associated with it */
	void ClearAssetData(const FString& InFilename);

private:
	FAssetDataCache										AssetDataCache;
	TArray<FString>										FileHistoryToUpdate;
	TArray<FString>										FilesToDownload;
	TQueue<FSourceControlStatePtr, EQueueMode::Spsc>	AssetDataToFetch;
	TQueue<FAssetDataToLoad, EQueueMode::Mpsc>			AssetDataToLoad;
	FDelegateHandle										ProviderChangedDelegateHandle;
	FDelegateHandle										ActorLabelChangedDelegateHandle;
	FDelegateHandle										AssetUpdatedDelegateHandle;
	std::atomic<uint32>									CurrentAsyncTask = 0;
	bool												bIsSourceControlDialogShown = false;
	bool												bIsUpdateHistoryInFlight = false;
	bool												bIsDownloadFileInFlight = false;
};
