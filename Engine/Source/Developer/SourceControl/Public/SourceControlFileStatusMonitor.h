// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Ticker.h"
#include "Delegates/DelegateCombinations.h"
#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#define UE_API SOURCECONTROL_API

class ISourceControlState;

/**
 * Periodically monitors the source controlled status of a collection of files and notifies the latest status, including
 * potential warnings such as 'out of date', 'locked by other', etc. This service is meant to throttle and batch non-time
 * sensitive FUpdateStatus requests to the source control provider because those requests are expensive and can add an
 * undesired load on the provider server.
 */
class FSourceControlFileStatusMonitor : public TSharedFromThis<FSourceControlFileStatusMonitor>
{
public:
	/** Invoked when the status of a source controlled files is updated. The state can be null when the source control provider is changed. */
	DECLARE_DELEGATE_TwoParams(FOnSourceControlFileStatus, const FString& /*AbsPathname*/, const ISourceControlState* /*State*/);

public:
	UE_API FSourceControlFileStatusMonitor();
	UE_API virtual ~FSourceControlFileStatusMonitor();

	/**
	 * Starts monitoring the source control status of the specified file. If the status is already known (the file is already monitored
	 * by another party), the know status is returned right away, otherwise a request is sent to the source control provider to get the
	 * status. The request is subject to the probation period policy to allow batching requests. The file status is refreshed according to
	 * the refresh period policy.
	 *
	 * The caller must be sure to call StopMonitoringFile() when the files status is not desired anymore. It is ok to call StopMonitoringFile()
	 * within the callback OnSourceControlledFileStatus().
	 *
	 * @param Owner The unique Id of the caller, typically the caller memory address.
	 * @param AbsPathname The absolute path and filname of the file to monitor.
	 * @param OnSourceControlledFileStatus Delegate invoked whe the status of the file is updated.
	 *
	 * @see SetNewRequestProbationPeriod()
	 * @see SetUpdateStatusPeriod()
	 * @see SetSuspendMonitoringPolicy()
	 * @see GetStatusAge()
	 */
	UE_API void StartMonitoringFile(uintptr_t OwnerId, const FString& AbsPathname, FOnSourceControlFileStatus OnSourceControlledFileStatus);
	UE_API void StartMonitoringFiles(uintptr_t OwnerId, const TArray<FString>& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus);
	UE_API void StartMonitoringFiles(uintptr_t OwnerId, const TSet<FString>& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus);

	/**
	 * Stops monitoring the source control status of the specified file. If the specified file is not monitored, the function
	 * returns successfully.
	 *
	 * @param OwnerId The unique Id of the caller, typically the caller memory address.
	 * @param AbsPathname The absolute path and filname of the file to stop monitoring.
	 *
	 * @note This can be called fron the FOnSourceControlFileStatus callback passed to StartMonitoringFile().
	 */
	UE_API void StopMonitoringFile(uintptr_t OwnerId, const FString& AbsPathname);
	UE_API void StopMonitoringFiles(uintptr_t OwnerId, const TArray<FString>& AbsPathnames);
	UE_API void StopMonitoringFiles(uintptr_t OwnerId, const TSet<FString>& AbsPathnames);

	/**
	 * Stops monitoring all the files that were registered by the specified owner Id.
	 */
	UE_API void StopMonitoringFiles(uintptr_t OwnerId);

	/**
	 * Starts monitoring files that weren't monitored yet by the specified owner, stops monitoring those that were monitored by the owner but are not in the
	 * updated list and keep monitoring files that were monitored before and still in the updated list.
	 *
	 * @param OwnerId The unique Id of the caller, typically the caller memory address.
	 * @param AbsPathnames The list of absolute file pathnames that must be monitored. (The set is modified during the operation)
	 * @param OnSourceControlledFileStatus Delegate invoked whe the status of the file is updated.
	 */
	UE_API void SetMonitoringFiles(uintptr_t OwnerId, TSet<FString>&& AbsPathnames, FOnSourceControlFileStatus OnSourceControlledFileStatus);

	/**
	 * Returns the set of files being monitored by the specifed owner.
	 */
	UE_API TSet<FString> GetMonitoredFiles(uintptr_t OwnerId);

	/**
	 * Returns the file status age, i.e. (now - last_status_update) if the file is
	 * monitored. If the initial status request hasn't been received yet, returns
	 * zero.
	 */
	UE_API TOptional<FTimespan> GetStatusAge(const FString& AbsPathname) const;

	/**
	 * Set the times waited before a newly added file status request is sent to the source control provider. This is used batch several
	 * status for file added closed in time into a single server request, for example if StartMonitoringFile() is called in a loop or
	 * when the user scroll a long list of files for which the status needs to be updated. Setting a value of 0 remove the probation period.
	 * By default, the propbation period is 1 second.
	 */
	void SetNewRequestProbationPeriodPolicy(const FTimespan& InPropbationPeriod) { ProbationPeriodPolicy = InPropbationPeriod; }
	FTimespan GetNewRequestProbationPeriodPolicy() const { return ProbationPeriodPolicy; }

	/**
	 * Set the status refresh period. By default, the monitored files are refreshed every 5 minutes, providing the monitor
	 * is it suspended according to the suspend monitor policy. The refresh period has an impact on the load of the source control
	 * provider. A short period may overload the provider service.
	 * 
	 * @see SetSuspendMonitoringPolicy
	 */
	void SetUpdateStatusPeriodPolicy(const FTimespan& InRefreshPeriod) { RefreshPeriodPolicy = InRefreshPeriod; }
	FTimespan GetUpdateStatusPeriodPolicy() const { return RefreshPeriodPolicy; }

	/**
	 * Sets the maximum number of files to batch into a single request. By default, a maximum of 100 files status
	 * is requested.
	 */
	void SetMaxFilePerRequestPolicy(int32 MaxNum) { MaxFileNumPerRequestPolicy = MaxNum; }
	int32 GetMaxFilePerRequestPolicy() const { return MaxFileNumPerRequestPolicy; }

	/**
	 * Sets a function invoked to detect if the monitor should suspend its activities, for example, if no user
	 * is currently interacting with the application, it may not worth checking the file status periodically.
	 * By default, the monitor will suspends  if there are not Slate interaction within 5 minutes and resume
	 * at the next user interaction.
	 */
	void SetSuspendMonitoringPolicy(TFunction<bool()> IsMonitoringSuspended) { SuspendMonitoringPolicy = MoveTemp(IsMonitoringSuspended); }

private:
	// Some compilers were ambigous when picking a Hash function for uintprt_t, provide a custom one.
	struct FUintptrHash : public TDefaultMapKeyFuncs<uintptr_t, FOnSourceControlFileStatus, false>
	{
		static uint32 GetKeyHash(const uintptr_t& Key)
		{
			return ::GetTypeHash(static_cast<uint64>(Key));
		}
	};

	struct FSourceControlFileStatus
	{
		FSourceControlFileStatus(uintptr_t OriginalOwnerId, FOnSourceControlFileStatus InFileStatusUpdateDelegate)
		{
			OwnerDelegateMap.Emplace(OriginalOwnerId, MoveTemp(InFileStatusUpdateDelegate));
		}

		TSharedPtr<ISourceControlState> FileState;
		double LastStatusCheckTimestampSecs = 0.0;
		TMap<uintptr_t, FOnSourceControlFileStatus, FDefaultSetAllocator, FUintptrHash> OwnerDelegateMap;
	};

private:
	/** Invoked when the source control has updated the status for the requested files. */
	UE_API void OnSourceControlStatusUpdate(const TSharedRef<ISourceControlOperation>& InOperation, ECommandResult::Type InType);

	/** Invoked when the source control provider changes. */
	UE_API void OnSourceControlProviderChanged(ISourceControlProvider& OldProvider, ISourceControlProvider& NewProvider);

	/** Invoked by the main loop ticker to fire source control requests. */
	UE_API bool Tick(float DeltaTime);

	/** Returns the specified files status or null if not found. */
	UE_API TSharedPtr<FSourceControlFileStatus> FindFileStatus(const FString& Pathname) const;

	/** Whethers the monitor has an in-flight request with the source control provider. */
	bool HasOngoingRequest() const { return RequestedStatusFiles.Num() > 0; }

private:
	/** The files currently monitored, mapping absolute file pathname to the monitored information. */
	TMap<FString, TSharedPtr<FSourceControlFileStatus>> MonitoredFiles;

	/** The list of files for which a request to the source control provider is currently in-flight/ongoing. */
	TArray<FString> RequestedStatusFiles;

	/** The newly added files probation period. Time left to batch several files into the same requests. */
	FTimespan ProbationPeriodPolicy;

	/** The remaining files retry period. Time left until we retry a file's CanQuerySourceControl status. */
	FTimespan RemainingPeriodPolicy;

	/** The status refresh period. Time left until we re-request a file status. */
	FTimespan RefreshPeriodPolicy;

	/** Invoked to detect wheter the monitoring is paused/suspended (no request to source control provider).*/
	TFunction<bool()> SuspendMonitoringPolicy;

	/** The maximum number of files allowed to be batched together for a request to the provider. */
	int32 MaxFileNumPerRequestPolicy = 100;

	/** Last time a request to the source control provider was issued. */
	double LastSourceControlCheckSecs = 0.0;

	/** Last time a file was added to the list of monitored files. */
	double LastAddedFileTimeSecs = 0.0;

	/** Last time some files were left remaining after inspection. */
	double LastRemainingFileTimeSecs = 0.0;

	/** The oldest status update time. */
	double OldestFileStatusTimeSecs = 0.0;

	/** Number of files newly added that are in probation. */
	int32 NewAddedFileCount = 0;

	/** Handle for the tick. */
	FTSTicker::FDelegateHandle TickerHandle;

	/** Handle of the registered source provider change delegate. */
	FDelegateHandle SourceControlProviderChangedDelegateHandle;
};

#undef UE_API
