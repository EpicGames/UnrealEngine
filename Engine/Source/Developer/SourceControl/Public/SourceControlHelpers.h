// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "ISourceControlProvider.h"
#include "ISourceControlRevision.h"
#include "ISourceControlState.h"
#include "Internationalization/Text.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

#include "SourceControlHelpers.generated.h"

#define UE_API SOURCECONTROL_API

class FAnnotationLine;
class FName;
class ISourceControlProvider;
class UPackage;
class USourceControlHelpers;
struct FAssetData;
struct FFrame;

/**
 * Snapshot of source control state of a file
 * @see	USourceControlHelpers::QueryFileState()
 */
USTRUCT(BlueprintType)
struct FSourceControlState
{
	GENERATED_BODY()

public:

	FSourceControlState() :
		bIsValid(false),
		bIsUnknown(false),
		bCanCheckIn(false),
		bCanCheckOut(false),
		bIsCheckedOut(false),
		bIsCurrent(false),
		bIsSourceControlled(false),
		bIsAdded(false),
		bIsDeleted(false),
		bIsIgnored(false),
		bCanEdit(false),
		bCanDelete(false),
		bIsModified(false),
		bCanAdd(false),
		bIsConflicted(false),
		bCanRevert(false),
		bIsCheckedOutOther(false)
	{}

	void SetFromStatus(const FSourceControlStatePtr& InStatus)
	{
		if (InStatus)
		{
			bIsValid = true;

			// Copy over state info
			// - make these assignments a method of FSourceControlState if anything else sets a state
			bIsUnknown = InStatus->IsUnknown();
			bIsSourceControlled = InStatus->IsSourceControlled();
			bCanCheckIn = InStatus->CanCheckIn();
			bCanCheckOut = InStatus->CanCheckout();
			bIsCheckedOut = InStatus->IsCheckedOut();
			bIsCurrent = InStatus->IsCurrent();
			bIsAdded = InStatus->IsAdded();
			bIsDeleted = InStatus->IsDeleted();
			bIsIgnored = InStatus->IsIgnored();
			bCanEdit = InStatus->CanEdit();
			bCanDelete = InStatus->CanDelete();
			bCanAdd = InStatus->CanAdd();
			bIsConflicted = InStatus->IsConflicted();
			bCanRevert = InStatus->CanRevert();
			bIsModified = InStatus->IsModified();
			bIsCheckedOutOther = InStatus->IsCheckedOutOther();
			bIsCheckedOutInOtherBranch = InStatus->IsCheckedOutInOtherBranch();
			bIsModifiedInOtherBranch = InStatus->IsModifiedInOtherBranch();
			Filename = InStatus->GetFilename();

			if (bIsCheckedOutOther)
			{
				InStatus->IsCheckedOutOther(&CheckedOutOther);
			}

			//Go through the history to find the most recent user
			const int32 HistorySize = InStatus->GetHistorySize();
			for ( int32 Index = 0; Index < HistorySize; Index++ )
			{
				if ( const ISourceControlRevision* Revision = InStatus->GetHistoryItem( Index ).Get() )
				{
					const FString User = Revision->GetUserName();
					if ( !User.IsEmpty() )
					{
						PreviousUser = User;
						break;
					}
				}
			}
		}
		else
		{
			bIsValid = false;
		}
	}

	/** Get the local filename that this state represents */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	FString Filename;

	/** Indicates whether this source control state has valid information (true) or not (false) */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsValid = false;

	/** Determine if we know anything about the source control state of this file */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsUnknown = false;

	/** Determine if this file can be checked in. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanCheckIn = false;

	/** Determine if this file can be checked out */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanCheckOut = false;

	/** Determine if this file is checked out */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsCheckedOut = false;

	/** Determine if this file is up-to-date with the version in source control */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsCurrent = false;

	/** Determine if this file is under source control */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsSourceControlled = false;

	/**
	 * Determine if this file is marked for add
	 * @note	if already checked in then not considered mid add
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsAdded = false;

	/** Determine if this file is marked for delete */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsDeleted = false;

	/** Determine if this file is ignored by source control */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsIgnored = false;

	/** Determine if source control allows this file to be edited */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanEdit = false;

	/** Determine if source control allows this file to be deleted. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanDelete = false;

	/** Determine if this file is modified compared to the version in source control. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsModified = false;

	/** 
	 * Determine if this file can be added to source control (i.e. is part of the directory 
	 * structure currently under source control) 
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanAdd = false;

	/** Determine if this file is in a conflicted state */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsConflicted = false;

	/** Determine if this file can be reverted, i.e. discard changes and the file will no longer be checked-out. */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bCanRevert = false;

	/** Determine if this file is checked out by someone else */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsCheckedOutOther = false;

	/**
	 * Get name of other user who this file already checked out or "" if no other user has it checked out
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	FString CheckedOutOther;

	/** Determine if this file is checked out by someone else in another branch */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsCheckedOutInOtherBranch = false;

	/** Determine if this file is modified by someone in another branch */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	bool bIsModifiedInOtherBranch = false;

	/** The username of the previous user to modify this file */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	FString PreviousUser;
};  // FSourceControlState


/** 
 * Delegate used for performing operation on files that may need a checkout, but before they are added to source control 
 * @param	InDestFile			The filename that was potentially checked out
 * @param	InFileDescription	Description of the file to display to the user, e.g. "Text" or "Image"
 * @param	OutFailReason		Text describing why the operation failed
 * @return true if the operation was successful
 */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FOnPostCheckOut, const FString& /*InDestFile*/, const FText& /*InFileDescription*/, FText& /*OutFailReason*/);

// For backwards compatibility
typedef class USourceControlHelpers SourceControlHelpers;

/**
 * Editor source control common functionality.
 *
 * @note Many of these source control methods use *smart* file strings which can be one of:
 *   - fully qualified path
 *   - relative path
 *   - long package name
 *   - asset path
 *   - export text path (often stored on clipboard)
 *
 *   For example:
 *	 - D:\Epic\Dev-Ent\Projects\Python3rdBP\Content\Mannequin\Animations\ThirdPersonIdle.uasset
 *	 - Content\Mannequin\Animations\ThirdPersonIdle.uasset
 *	 - /Game/Mannequin/Animations/ThirdPersonIdle
 *	 - /Game/Mannequin/Animations/ThirdPersonIdle.ThirdPersonIdle
 *	 - AnimSequence'/Game/Mannequin/Animations/ThirdPersonIdle.ThirdPersonIdle'
 */
UCLASS(MinimalAPI, Abstract, Transient, meta = (ScriptName = "SourceControl"))
class USourceControlHelpers : public UObject
{	// Note - would use UBlueprintFunctionLibrary however it requires Engine module which breaks some
	// other modules that depend on the SourceControl module and expect to be Engine independent.

	GENERATED_BODY()

public:
	/**
	 * Determine the name of the current source control provider.
	 * @return	the name of the current source control provider. If one is not set then "None" is returned.
	 */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API FString CurrentProvider();

	/**
	 * Determine if there is a source control system enabled
	 * @return	true if enabled, false if not
	 */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool IsEnabled();

	/**
	 * Quick check if currently set source control provider is enabled and available for use
	 * (server-based providers can use this to return whether the server is available or not)
	 *
	 * @return	true if source control is available, false if it is not
	 */
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool IsAvailable();

	/**
	* Get status text set by SourceControl system if an error occurs regardless whether bSilent is set or not.
	* Only set if there was an error.
	*/
	UFUNCTION(BlueprintPure, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API FText LastErrorMsg();

	/**
	 * Use currently set source control provider to sync a file or directory to the head revision.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile	The file or directory to sync - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool SyncFile(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to sync files or directories to the head revision.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles	Files or directories to sync - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool SyncFiles(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Use currently set source control provider to check out a file.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile		The file to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckOutFile(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to check out specified files.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles		Files to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckOutFiles(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Use currently set source control provider to check out file or mark it for add.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile		The file to check out/add - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckOutOrAddFile(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to check out files or mark them for add.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles		The files to check out/add - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckOutOrAddFiles(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Helper function perform an operation on files in our 'source controlled' directories, handling checkout/add etc.
	 * @note	Blocks until action is complete. Older C++ only version of CheckOutOrAddFile().
	 *
	 * @param	InDestFile			The path to the destination file
	 * @param	InFileDescription	Description of the file to display to the user, e.g. "Text" or "Image"
	 * @param	OnPostCheckOut		Delegate used for performing operation on files that may need a checkout, but before they are added to source control 
	 * @param	OutFailReason		Text describing why the operation failed
	 * @return	Success or failure of the operation
	 * @see     CheckOutOrAddFile()
	 */
	static UE_API bool CheckoutOrMarkForAdd(const FString& InDestFile, const FText& InFileDescription, const FOnPostCheckOut& OnPostCheckOut, FText& OutFailReason);

	/**
	 * Use currently set source control provider to mark a file for add. Does nothing (and returns true) if the file is already under SC
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile		The file to add - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool MarkFileForAdd(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to mark files for add. Does nothing (and returns true) for any file that is already under SC
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles		Files to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool MarkFilesForAdd(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Use currently set source control provider to remove file from source control and
	 * delete the file.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile		The file to delete - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool MarkFileForDelete(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to remove files from source control and delete the files.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile		The file to delete - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent		if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool MarkFilesForDelete(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Use currently set source control provider to revert a file regardless whether any changes will be lost or not.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile	The file to revert - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool RevertFile(const FString& InFile, bool bSilent = false);

#if WITH_EDITOR

	/**
	 * Applies the provided function and optionally reloads the world
	 * @param	InPackagesToApplyOperation			The files/packages to apply the operation
	 * @param	InOperation							The function to apply
	 * @param	bReloadWorld						Reload the world
	 * @param	bInteractive						Whether to prompt user when discarding changes
	 * @return true if succeeded.
	 */
	static UE_API bool ApplyOperationAndReloadPackages(const TArray<FString>& InPackagesToApplyOperation, 
		const TFunctionRef<bool(const TArray<FString>&)>& InOperation, bool bReloadWorld = false, bool bInteractive = true);

	/**
     * Reverts the provided files then reloads packages.
	 * @param	InPackagesToRevert					The packages to revert
	 * @param	bRevertAll							Whether to revert all files
	 * @param	bReloadWorld						Reload the world
     * @return true if succeeded.
     */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool RevertAndReloadPackages(const TArray<FString>& InPackagesToRevert, bool bRevertAll = false, bool bReloadWorld = false);

	/**
	 * Reverts all changes and then reloads the world.
	 * @return true if succeeded.
	 */
	static UE_API bool RevertAllChangesAndReloadWorld();

#endif // !WITH_EDITOR

	/**
	 * Use currently set source control provider to revert files regardless whether any changes will be lost or not.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles	Files to revert - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool RevertFiles(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Use currently set source control provider to revert a file provided no changes have been made.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile	File to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool RevertUnchangedFile(const FString& InFile, bool bSilent = false);

	/**
	 * Use currently set source control provider to revert files provided no changes have been made.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles	Files to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent	if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool RevertUnchangedFiles(const TArray<FString>& InFiles, bool bSilent = false);

	/**
	 * Helper function to get a list of files that are unchanged & revert them. This runs synchronous commands.
	 * @note	Older C++ only version of RevertFiles(Files, false).
	 * @see		RevertFiles()
	 *
	 * @param	InProvider	The provider to use
	 * @param	InFiles		The files to operate on
	 */
	static UE_API void RevertUnchangedFiles(ISourceControlProvider& InProvider, const TArray<FString>& InFiles);

	/**
	 * Use currently set source control provider to check in a file.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile			The file to check in - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	InDescription	Description for check in
	 * @param	bSilent			if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @param	bKeepCheckedOut Keep files checked-out after checking in. This is helpful for maintaining "ownership" of files if further operations are needed.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckInFile(const FString& InFile, const FString& InDescription, bool bSilent = false, bool bKeepCheckedOut = false);

	/**
	 * Use currently set source control provider to check in specified files.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFiles			Files to check out - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	InDescription	Description for check in
	 * @param	bSilent			if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @param	bKeepCheckedOut Keep files checked-out after checking in. This is helpful for maintaining "ownership" of files if further operations are needed.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CheckInFiles(const TArray<FString>& InFiles, const FString& InDescription, bool bSilent = false, bool bKeepCheckedOut = false);

	/**
	 * Use currently set source control provider to copy a file.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InSourceFile	Source file string to copy from - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	InDestFile		Source file string to copy to - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard). If package, then uses same extension as source file.
	 * @param	bSilent			if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	true if succeeded, false if failed and can call LastErrorMsg() for more info.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API bool CopyFile(const FString& InSourceFile, const FString& InDestFile, bool bSilent = false);

	/**
	 * Helper function to copy a file into our 'source controlled' directories, handling checkout/add etc.
	 * @note	Blocks until action is complete. Older C++ only version of CopyFile().
	 *
	 * @param	InDestFile			The path to the destination file
	 * @param	InSourceFile		The path to the source file
	 * @param	InFileDescription	Description of the file to display to the user, e.g. "Text" or "Image"
	 * @param	OutFailReason		Text describing why the operation failed
	 * @return	Success or failure of the operation
	 */
	static UE_API bool CopyFileUnderSourceControl(const FString& InDestFile, const FString& InSourceFile, const FText& InFileDescription, FText& OutFailReason);

	/**
	 * Use currently set source control provider to query a file's source control state.
	 * @note	Blocks until action is complete.
	 *
	 * @param	InFile			The file to query - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	 * @param	bSilent			if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @return	Source control state - see USourceControlState. It will have bIsValid set to false if
	 *			it could not have its values set.
	 */
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API FSourceControlState QueryFileState(const FString& InFile, bool bSilent = false, bool bUseSourceControlStateCache = false);

	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API TArray<FSourceControlState> QueryFileStates(TArray<FString> InFiles, bool bSilent = false, bool bUseSourceControlStateCache = false);

	//Delegate to broadcast FileState upon AsyncQueryFileState completion
	DECLARE_DYNAMIC_DELEGATE_OneParam(FQueryFileStateDelegate, FSourceControlState, FileStateOut);
	/**
	* Query the source control state of the specified file, asynchronously.
	*
	* @param	FileStateCallback Source control state - see USourceControlState. It will have bIsValid set to false if it could not have its values set.
	* @param	InFile			  The file to query - can be either fully qualified path, relative path, long package name, asset path or export text path (often stored on clipboard)
	* @param	bSilent			  if false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	*/
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API void AsyncQueryFileState(FQueryFileStateDelegate FileStateCallback, const FString& InFile, bool bSilent = false);
	UFUNCTION(BlueprintCallable, Category = "Editor Scripting | Editor Revision Control Helpers", meta = (Keywords = "Source Control"))
	static UE_API void AsyncQueryFileStates(FQueryFileStateDelegate FileStateCallback, TArray<FString> InFiles, bool bSilent = false);
	
	/**
	 * Use currently set source control provider to query the list of files in the depot under a certain path.
	 * @note	Blocks until action is complete.
	 *
	 * @param	Path				The path which we want to query the list of files from.
	 * @param	OutFilesList		An array containing the list of files under the queried path.
	 * @param	bIncludeDeleted		Include files that have been deleted from the depot.
	 * @param	bSilent				If false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @param	bIsFileRegexSearch	If false, it is a folder search. If true, it is a file regex search
	 * @return	Success or failure of the operation
	 */
	static UE_API bool GetFilesInDepotAtPath(const FString& Path, TArray<FString>& OutFilesList, bool bIncludeDeleted = false, bool bSilent = false, bool bIsFileRegexSearch = false);

	/**
	 * Use currently set source control provider to query the list of files in the depot under a list of paths.
	 * @note	Blocks until action is complete.
	 *
	 * @param	Paths				The paths which we want to query the list of files from.
	 * @param	OutFilesList		An array containing the list of files under the queried path.
	 * @param	bIncludeDeleted		Include files that have been deleted from the depot.
	 * @param	bSilent				If false (default) then write out any error info to the Log. Any error text can be retrieved by LastErrorMsg() regardless.
	 * @param	bIsFileRegexSearch	If false, it is a folder search. If true, it is a file regex search
	 * @return	Success or failure of the operation
	 */
	static UE_API bool GetFilesInDepotAtPaths(const TArray<FString>& Paths, TArray<FString>& OutFilesList, bool bIncludeDeleted = false, bool bSilent = false, bool bIsFileRegexSearch = false);

	/**
	 * Helper function to get a filename for a package name.
	 * @param	InPackageName	The package name to get the filename for
	 * @return the filename
	 */
	static UE_API FString PackageFilename(const FString& InPackageName);

	/**
	 * Helper function to get a filename for a package.
	 * @param	InPackage	The package to get the filename for
	 * @return the filename
	 */
	static UE_API FString PackageFilename(const UPackage* InPackage);

	/**
	 * Helper function to convert package name array into a filename array.
	 * @param	InPackageNames	The package name array
	 * @return an array of filenames
	 */
	static UE_API TArray<FString> PackageFilenames(const TArray<FString>& InPackageNames);

	/**
	 * Helper function to convert package array into filename array.
	 * @param	InPackages	The package array
	 * @return an array of filenames
	 */
	static UE_API TArray<FString> PackageFilenames(const TArray<UPackage*>& InPackages);

	/**
	 * Helper function to convert a filename array to absolute paths.
	 * @param	InFileNames	The filename array
	 * @return an array of filenames, transformed into absolute paths
	 */
	static UE_API TArray<FString> AbsoluteFilenames(const TArray<FString>& InFileNames);

	/**
	 * Helper function to annotate a file using a label
	 * @param	InProvider	The provider to use
	 * @param	InLabel		The label to use to retrieve the file
	 * @param	InFile		The file to annotate
	 * @param	OutLines	Output array of annotated lines
	 * @returns true if successful
	 */
	static UE_API bool AnnotateFile(ISourceControlProvider& InProvider, const FString& InLabel, const FString& InFile, TArray<FAnnotationLine>& OutLines);

	/**
	 * Helper function to annotate a file using a changelist/checkin identifier
	 * @param	InProvider				The provider to use
	 * @param	InCheckInIdentifier		The changelist/checkin identifier to use to retrieve the file
	 * @param	InFile					The file to annotate
	 * @param	OutLines				Output array of annotated lines
	 * @returns true if successful
	 */
	static UE_API bool AnnotateFile(ISourceControlProvider& InProvider, int32 InCheckInIdentifier, const FString& InFile, TArray<FAnnotationLine>& OutLines);

	/**
	 * Helper function to branch/integrate packages from one location to another maintaining
	 * a relationship between the files in source control (when possible)
	 * @param	DestPackage			The destination package
	 * @param	SourcePackage		The source package
	 * @Param	StateCacheUsage		Whether to use the source control state cache
	 * @return true if the file packages were successfully branched.
	 */
	static UE_API bool BranchPackage(UPackage* DestPackage, UPackage* SourcePackage, EStateCacheUsage::Type StateCacheUsage = EStateCacheUsage::ForceUpdate);

	/**
	 * Helper function to copy a package from one location to another
	 * @param	DestPackage             The destination package
	 * @param	SourcePackage           The source package
	 * @Param	StateCacheUsage         Whether to use the source control state cache
	 * @return true if the file packages were successfully branched.
	 */
	static UE_API bool CopyPackage(UPackage* DestPackage, UPackage* SourcePackage, EStateCacheUsage::Type StateCacheUsage = EStateCacheUsage::ForceUpdate);

	/**
	 * Helper function to get the ini filename for storing source control settings
	 * @return the filename
	 */
	static UE_API const FString& GetSettingsIni();

	/**
	 * Helper function to get the ini filename for storing global source control settings
	 * @return the filename
	 */
	static UE_API const FString& GetGlobalSettingsIni();

	/**
	* Helper function to retrieve the FAssetData associated with a given file
	* @return Whether the asset data could be retrieved or not
	*/
	static UE_API bool GetAssetData(const FString& InFileName, const FString& InPackageName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies = nullptr);

	/**
	 * Helper function to retrieve the FAssetData associated with a given file
	 * @return Whether the asset data could be retrieved or not
	 */
	static UE_API bool GetAssetData(const FString& InFileName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies = nullptr);

	/**
	* Helper function to retrieve the FAssetData associated with a given package name
	* @return Whether the asset data could be retrieved or not
	*/
	static UE_API bool GetAssetDataFromPackage(const FString& InPackageName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies = nullptr);

	/**
	 * Helper function to get the asset data from a file's history
	 * Note: will not query history to prevent recursion
	 */
	static UE_API bool GetAssetDataFromFileHistory(const FString& InFileName, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies = nullptr, int64 MaxFetchSize = -1);

	/**
	 * Helper function to get the asset data from a file's history
	 * Note: will not query history to prevent recursion
	 */
	static UE_API bool GetAssetDataFromFileHistory(FSourceControlStatePtr InSourceControlState, TArray<FAssetData>& OutAssets, TArray<FName>* OutDependencies = nullptr, int64 MaxFetchSize = -1);

	/**
	 * Find packages that can be reverted in Source Control locations
	 */
	static UE_API bool ListRevertablePackages(TArray<FString>& OutRevertablePackageNames);

	/**
	 * Get the list of files and directories that source control should check when looking for changes.
	 *
	 * @param	bContentOnly	True to only include content directories.
	 */
	static UE_API TArray<FString> GetSourceControlLocations(const bool bContentOnly = false);

};  // USourceControlHelpers


/** 
 * Helper class that ensures FSourceControl is properly initialized and shutdown by calling Init/Close in
 * its constructor/destructor respectively.
 */
class FScopedSourceControl
{
public:
	/** Constructor; Initializes Source Control Provider */
	UE_API FScopedSourceControl();

	/** Destructor; Closes Source Control Provider */
	UE_API ~FScopedSourceControl();

	/** Get the provider we are using */
	UE_API ISourceControlProvider& GetProvider();

private:
	bool bInitSourceControl = false;
};

#undef UE_API
