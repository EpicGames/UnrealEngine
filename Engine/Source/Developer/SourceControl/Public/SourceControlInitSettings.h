// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"

#define UE_API SOURCECONTROL_API

class FString;

class FSourceControlInitSettings
{
public:
	enum class EBehavior
	{
		/** All existing settings will be overridden via the contents of FSourceControlInitSettings. Settings that are not found will be reset to default states */
		OverrideAll,
		/** Only the settings found in FSourceControlInitSettings will be overridden. Settings not found will be left with their current values. */
		OverrideExisting,
	};

	enum class ECmdLineFlags : uint8
	{
		/** Do not read any settings from the commandline */
		None,
		/** Read all available settings from the commandline */
		ReadAll
	};

	enum class EConfigBehavior : uint8
	{
		/** Can both read from, and save to the ini file*/
		ReadWrite,
		/** Will only read settings from the ini file, settings determined at runtime will not be saved to the ini file */
		ReadOnly,
		/** The settings will not be saved to the ini file, nor will they be read from the ini file  */
		None
	};

	UE_API FSourceControlInitSettings(EBehavior Behavior);
	UE_API FSourceControlInitSettings(EBehavior Behavior, ECmdLineFlags CmdLineFlags);
	~FSourceControlInitSettings() = default;

	UE_API void SetConfigBehavior(EConfigBehavior Behavior);

	UE_API bool CanWriteToConfigFile() const;
	UE_API bool CanReadFromConfigFile() const;
		
	UE_API void AddSetting(FStringView SettingName, FStringView SettingValue);
	UE_API void OverrideSetting(FStringView SettingName, FString& InOutSettingValue);

	UE_API bool HasOverrides() const;
	UE_API bool IsOverridden(FStringView SettingName) const;

	UE_API void SetCmdLineFlags(ECmdLineFlags Flags);
	UE_API bool ShouldReadFromCmdLine() const;

private:

	EBehavior OverrideBehavior		= EBehavior::OverrideAll;
	ECmdLineFlags CmdLineFlags		= ECmdLineFlags::None;
	EConfigBehavior ConfigBehavior	= EConfigBehavior::ReadWrite;

	TMap<FString, FString> Settings;
};

#undef UE_API
