// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlOperation.h"

/** Adds some common functionality to source control operations. */
class FSourceControlOperationBase : public ISourceControlOperation
{
public:
	enum class EFlags
	{
		/* No specialization applied */
		None = 0,
		/* When set the operation should not log any errors but should continue to store them in the ResultInfo member */
		DisableErrorLogging = 1 << 0,
		/* When set the operation should not log any info messages but should continue to store them in the ResultInfo member */
		DisableInfoLogging = 1 << 1,
	};
	FRIEND_ENUM_CLASS_FLAGS(EFlags)

	FSourceControlOperationBase() = default;
	virtual ~FSourceControlOperationBase() = default;

	/** Retrieve any info or error messages that may have accumulated during the operation. */
	virtual const FSourceControlResultInfo& GetResultInfo() const override
	{
		return ResultInfo;
	}

	/** Add info/warning message. */
	virtual void AddInfoMessge(const FText& InInfo)
	{
		ResultInfo.InfoMessages.Add(InInfo);
	}

	/** Add error message. */
	virtual void AddErrorMessge(const FText& InError)
	{
		ResultInfo.ErrorMessages.Add(InError);
	}

	/** Add tag. */
	virtual void AddTag(const FString& InTag)
	{
		ResultInfo.Tags.Add(InTag);
	}

	/**
	 * Append any info or error messages that may have accumulated during the operation prior
	 * to returning a result, ensuring to keep any already accumulated info.
	 */
	virtual void AppendResultInfo(const FSourceControlResultInfo& InResultInfo) override
	{
		ResultInfo.Append(InResultInfo);
	}

	void SetEnableErrorLogging(bool bEnabled)
	{
		if (bEnabled)
		{
			EnumRemoveFlags(Flags, EFlags::DisableErrorLogging);
		}
		else
		{
			EnumAddFlags(Flags, EFlags::DisableErrorLogging);
		}
	}

	virtual bool ShouldLogErrors() const override
	{
		return !EnumHasAllFlags(Flags, EFlags::DisableErrorLogging);
	}

	void SetEnableInfoLogging(bool bEnabled)
	{
		if (bEnabled)
		{
			EnumRemoveFlags(Flags, EFlags::DisableInfoLogging);
		}
		else
		{
			EnumAddFlags(Flags, EFlags::DisableInfoLogging);
		}
	}

	virtual bool ShouldLogInfos() const override
	{
		return !EnumHasAllFlags(Flags, EFlags::DisableInfoLogging);
	}

	FSourceControlResultInfo ResultInfo;

protected:
	EFlags Flags = EFlags::None;
};

ENUM_CLASS_FLAGS(FSourceControlOperationBase::EFlags);
