// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "HAL/PlatformCrt.h"
#include "ISourceControlChangelist.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Memory/SharedBuffer.h"
#include "SourceControlOperationBase.h"
#include "SourceControlPreferences.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

#define LOCTEXT_NAMESPACE "SourceControl"

/**
 * Operation used to connect (or test a connection) to source control
 */
class FConnect : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Connect";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Connecting", "Connecting to Revision Control...");
	}

	const FString& GetPassword() const
	{
		return Password;
	}

	void SetPassword(const FString& InPassword)
	{
		Password = InPassword;
	}

	const FText& GetErrorText() const
	{
		return OutErrorText;
	}

	void SetErrorText(const FText& InErrorText)
	{
		OutErrorText = InErrorText;
	}

protected:
	/** Password we use for this operation */
	FString Password;

	/** Error text for easy diagnosis */
	FText OutErrorText;
};

/**
 * Operation used to check files into source control
 */
class FCheckIn : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "CheckIn";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_CheckIn", "Checking file(s) into Revision Control...");
	}

	void SetDescription( const FText& InDescription )
	{
		Description = InDescription;
	}

	const FText& GetDescription() const
	{
		return Description;
	}

	void SetSuccessMessage( const FText& InSuccessMessage )
	{
		SuccessMessage = InSuccessMessage;
	}

	const FText& GetSuccessMessage() const
	{
		return SuccessMessage;
	}

	void SetKeepCheckedOut( const bool bInKeepCheckedOut )
	{
		bKeepCheckedOut = bInKeepCheckedOut;
	}

	bool GetKeepCheckedOut()
	{
		return bKeepCheckedOut;
	}

protected:
	/** Description of the checkin */
	FText Description;

	/** A short message listing changelist/revision we submitted, if successful */
	FText SuccessMessage;

	/** Keep files checked-out after checking in */
	bool bKeepCheckedOut = false;
};

/**
 * Operation used to check files out of source control
 */
class FCheckOut : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "CheckOut";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_CheckOut", "Checking file(s) out of Revision Control...");
	}
};

/**
 * Operation used to get a file list out of source control
 */
class FGetFileList : public FSourceControlOperationBase
{
public:
	enum EGetFileListMethod
	{
		FolderSearch,
		FileRegexSearch // Note: Only works for Perforce for now
	};

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetFileList";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetFileList", "Getting file list out of Revision Control...");
	}

	void SetIncludeDeleted( const bool bInIncludeDeleted )
	{
		bIncludeDeleted = bInIncludeDeleted;
	}

	bool GetIncludeDeleted()
	{
		return bIncludeDeleted;
	}

	void SetQuiet(bool bInQuiet)
	{
		bForceQuiet = bInQuiet;
	}

	bool ShouldBeQuiet() const
	{
		return bForceQuiet;
	}

	void SetMethodUsed(const EGetFileListMethod& InMethodUsed)
	{
		MethodUsed = InMethodUsed;
	}

	EGetFileListMethod GetMethodUsed()
	{
		return MethodUsed;
	}

	const TArray<FString>& GetFilesList() const
	{
		return FilesList;
	}

	void SetFilesList(TArray<FString>&& InFilesList)
	{
		FilesList = MoveTemp(InFilesList);
	}

	const TArray<FString>& GetSearchPatterns() const
	{
		return SearchPatterns;
	}

	void SetSearchPattern(const TArray<FString>& InSearchPatterns)
	{
		SearchPatterns = InSearchPatterns;
	}

protected:
	/** Method used to get file list */
	EGetFileListMethod MethodUsed = EGetFileListMethod::FolderSearch;

	/** Include deleted files in the list. */
	bool bIncludeDeleted = false;

	/** Force disable any logs generated by that operation. */
	bool bForceQuiet = false;

	/** Stored result of the operation */
	TArray<FString> FilesList;

	/** The search patterns for the file list */
	TArray<FString> SearchPatterns;
};

/**
 * Operation used to mark files for add in source control
 */
class FMarkForAdd : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "MarkForAdd";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Add", "Adding file(s) to Revision Control...");
	}
};

/**
 * Operation used to mark files for delete in source control
 */
class FDelete : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Delete";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Delete", "Deleting file(s) from Revision Control...");
	}
};

/**
 * Operation used to revert changes made back to the state they are in source control
 */
class FRevert : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Revert";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Revert", "Reverting file(s) in Revision Control...");
	}

	void SetSoftRevert(const bool bInSoftRevert)
	{
		bIsSoftRevert = bInSoftRevert;
	}

	bool IsSoftRevert() const
	{
		return bIsSoftRevert;
	}

	void SetRevertAll(const bool bInRevertAll)
	{
		bIsRevertAll = bInRevertAll;
	}

	bool IsRevertAll() const
	{
		return bIsRevertAll;
	}

	bool ShouldDeleteNewFiles() const
	{
		return USourceControlPreferences::ShouldDeleteNewFilesOnRevert();
	}

	const TArray<FString>& GetDeletedFiles() const { return DeletedFiles; }

	void AddDeletedFile(const FString& InDeletedFile)
	{
		DeletedFiles.Add(InDeletedFile);
	}

protected:
	bool				bIsSoftRevert = false;
	bool				bIsRevertAll = false;
	TArray<FString>		DeletedFiles;
};

/**
 * Operation used to determine the files that would be affected by a sync operation
 */
class FSyncPreview : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "SyncPreview";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_SyncPreview", "Previewing sync file(s) from Revision Control...");
	}

	void SetRevision(const FString& InRevision)
	{
		Revision = InRevision;
	}

	const FString& GetRevision() const
	{
		return Revision;
	}

	void SetHeadRevisionFlag(const bool bInHeadRevision)
	{
		bHeadRevision = bInHeadRevision;
	}

	bool IsHeadRevisionFlagSet() const
	{
		return bHeadRevision;
	}

	void SetTransferSize(const int64 InTransferSize)
	{
		TransferSize = InTransferSize;
	}

	int64 GetTransferSize() const
	{
		return TransferSize;
	}

	void SetAffectedFiles(TArray<FString>&& InAffectedFiles)
	{
		AffectedFiles = MoveTemp(InAffectedFiles);
	}

	const TArray<FString>& GetAffectedFiles() const
	{
		return AffectedFiles;
	}
	
protected:
	/** Target Revision to which the sync preview refers to */
	FString Revision;

	/** Flag abstracting if the operation aims to preview a sync to head */
	bool bHeadRevision = false;

	/** Number of bytes that need to be transferred for the sync operation */
	int64 TransferSize = 0;

	/** Array of files that would be affected by the sync operation */
	TArray<FString> AffectedFiles;
};

/**
 * Operation used to sync files to the state they are in source control
 */
class FSync : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Sync";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Sync", "Syncing file(s) from Revision Control...");
	}

	UE_DEPRECATED(4.26, "FSync::SetRevisionNumber(int32) has been deprecated. Please update to Fsync::SetRevision(const FString&).")
	void SetRevisionNumber(int32 InRevisionNumber)
	{
		SetRevision(FString::Printf(TEXT("%d"), InRevisionNumber));
	}
	void SetRevision( const FString& InRevision )
	{
		Revision = InRevision;
	}

	const FString& GetRevision() const
	{
		return Revision;
	}

	void SetHeadRevisionFlag(const bool bInHeadRevision)
	{
		bHeadRevision = bInHeadRevision;
	}

	bool IsHeadRevisionFlagSet() const
	{
		return bHeadRevision;
	}

	void SetForce(const bool bInForce)
	{
		bForce = bInForce;
	}

	bool IsForced() const
	{
		return bForce;
	}

	void SetLastSyncedFlag(const bool bInLastSynced)
	{
		bLastSynced = bInLastSynced;
	}

	bool IsLastSyncedFlagSet() const
	{
		return bLastSynced;
	}

protected:
	/** Revision to sync to */
	FString Revision;

	/** Flag abstracting if the operation aims to sync to head */
	bool bHeadRevision = false;

	/** Flag abstracting if the operation aims to force sync to last synced revision */
	bool bLastSynced = false;

	/** Forces operation, even if the file is already at the wanted revision. */
	bool bForce = false;
};

/**
 * Operation used to update the source control status of files
 */
class FUpdateStatus : public FSourceControlOperationBase
{
public:
	FUpdateStatus()
		: bUpdateHistory(false)
		, bGetOpenedOnly(false)
		, bUpdateModifiedState(false)
		, bUpdateModifiedStateToLocalRevision(false)
		, bCheckingAllFiles(false)
		, bForceQuiet(false)
		, bForceUpdate(false)
		, bSetRequireDirPathEndWithSeparator(false)
	{
	}

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "UpdateStatus";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Update", "Updating file(s) Revision Control status...");
	}

	void SetUpdateHistory( bool bInUpdateHistory )
	{
		bUpdateHistory = bInUpdateHistory;
	}

	void SetGetOpenedOnly( bool bInGetOpenedOnly )
	{
		bGetOpenedOnly = bInGetOpenedOnly;
	}

	void SetUpdateModifiedState( bool bInUpdateModifiedState )
	{
		bUpdateModifiedState = bInUpdateModifiedState;
	}

	void SetUpdateModifiedStateToLocalRevision(bool bInUpdateModifiedStateToLocalRevision)
	{
		bUpdateModifiedStateToLocalRevision = bInUpdateModifiedStateToLocalRevision;
	}

	void SetCheckingAllFiles( bool bInCheckingAllFiles )
	{
		bCheckingAllFiles = bInCheckingAllFiles;
	}

	void SetQuiet(bool bInQuiet)
	{
		bForceQuiet = bInQuiet;
	}

	/** 
	 * Sets the method that the operation will use to determine if a path
	 * references a file or a directory. For more details @see IsDirectoryPath()
	 * 
	 * @param bFlag When true the operation will check the path and assume that it
	 *				is a directory if the path ends with '/' or '\' and a file if it 
	 *				does not.
	 *				When false (the default) the operation will poll the file system
	 *				with the path to see if it is a file or a directory.
	 * 
	 */
	void SetRequireDirPathEndWithSeparator(bool bFlag)
	{
		bSetRequireDirPathEndWithSeparator = bFlag;
	}

	void SetForceUpdate(const bool bInForceUpdate)
	{
		bForceUpdate = bInForceUpdate;
	}

	bool ShouldUpdateHistory() const
	{
		return bUpdateHistory;
	}

	bool ShouldGetOpenedOnly() const
	{
		return bGetOpenedOnly;
	}

	bool ShouldUpdateModifiedState() const
	{
		return bUpdateModifiedState;
	}

	bool ShouldUpdateModifiedStateToLocalRevision() const
	{
		return bUpdateModifiedStateToLocalRevision;
	}

	bool ShouldCheckAllFiles() const
	{
		return bCheckingAllFiles;
	}

	bool ShouldBeQuiet() const
	{
		return bForceQuiet;
	}

	bool ShouldForceUpdate() const
	{
		return bForceUpdate;
	}

	/** 
	 * Returns if the given path should be considered a directory or not.
	 * If bSetRequireDirPathEndWithSeparator is not set (the default) then
	 * we will poll the file system. However in some cases this can be very
	 * slow, in which case bSetRequireDirPathEndWithSeparator can be set 
	 * to true and we require that any directory path must be terminated
	 * by a path separator (/ or \) so that we can tell by simply looking at
	 * the path itself.
	 * This is opt in behavior to avoid breaking existing 3rd party code that
	 * relies on the default behavior.
	 */
	SOURCECONTROL_API bool IsDirectoryPath(const FString& Path) const;

protected:
	/** Whether to update history */
	bool bUpdateHistory;

	/** Whether to just get files that are opened/edited */
	bool bGetOpenedOnly;

	/** Whether to update the modified state - expensive */
	bool bUpdateModifiedState;

	/** Whether to update the modified state against local revision - only used when bUpdateModifiedState is true */
	bool bUpdateModifiedStateToLocalRevision;

	/** Hint that we are intending on checking all files in the project - some providers can optimize for this */
	bool bCheckingAllFiles;

	/** Controls whether the operation will trigger an update or not */
	bool bForceQuiet;

	/** Forces the verification for provided files - providers can ignore files not opened/edited without it */
	bool bForceUpdate;

	/** If we should assume paths ending in a separator are directory paths or do we need to check with the file system? */
	bool bSetRequireDirPathEndWithSeparator;
};

/**
 * Operation used to copy a file or directory from one location to another
 */
class FCopy : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Copy";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Copy", "Copying file(s) in Revision Control...");
	}

	void SetDestination(const FString& InDestination)
	{
		Destination = InDestination;
	}

	void SetDestination(FString&& InDestination)
	{
		Destination = MoveTemp(InDestination);
	}

	const FString& GetDestination() const
	{
		return Destination;
	}

	enum class ECopyMethod
	{
		Branch,  // The new file is branched from the original file
		Add      // The new file has no relation to the original file
	};

	enum class EResolveMethod
	{
		AcceptTarget,	// Resolve by accepting the destination file from the copy operation
		AcceptSource,	// Resolve by accepting the source file from the copy operation
	};

	/** Whether a relationship to the original file should be maintained */
	ECopyMethod CopyMethod = ECopyMethod::Branch;

	/** The current state of the destination file */
	EResolveMethod ResolveMethod = EResolveMethod::AcceptTarget;

protected:
	/** Destination path of the copy operation */
	FString Destination;
};

/**
 * Operation used to resolve a file that is in a conflicted state.
 */
class FResolve : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		  return "Resolve";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Resolve", "Resolving file(s) in Revision Control...");
	}
};

/**
 * Operation used to retrieve pending changelist(s).
 */
class FGetPendingChangelists : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetPendingChangelists";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetPendingChangelists", "Retrieving pending changelist(s) from Revision Control...");
	}
};

/**
 * Operation used to retrieve submitted changelist(s).
 */
class FGetSubmittedChangelists : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetSubmittedChangelists"; 
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetSubmittedChangelists", "Retrieving submitted changelist(s) from Revision Control...");
	}

public:
	void SetDateFromFilter(const FDateTime& InDateFrom)
	{
		DateFrom = InDateFrom;
	}

	void SetDateToFilter(const FDateTime& InDateTo)
	{
		DateTo = InDateTo;
	}

	void SetOwnedFilter(bool InOwned)
	{
		Owned = InOwned;
	}

	void SetPaginationLimit(int32 InLimit)
	{
		if (InLimit > 0)
		{
			PaginationLimit = InLimit;
		}
	}

	void SetPaginationOffset(int32 InOffset)
	{
		if (InOffset >= 0)
		{
			PaginationOffset = InOffset;
		}
	}

	bool ShouldFilterByDateFrom(FDateTime& OutDateFrom)
	{
		if (DateFrom != FDateTime::MinValue())
		{
			OutDateFrom = DateFrom;
			return true;
		}
		return false;
	}

	bool ShouldFilterByDateTo(FDateTime& OutDateTo)
	{
		if (DateTo != FDateTime::MinValue())
		{
			OutDateTo = DateTo;
			return true;
		}
		return false;
	}

	bool ShouldFilterByOwned()
	{
		return Owned;
	}

	bool ShouldUsePaginationLimit(int32& OutPaginationLimit)
	{
		if (PaginationLimit != -1)
		{
			OutPaginationLimit = PaginationLimit;
			return true;
		}
		return false;
	}

	bool ShouldUsePaginationOffset(int32& OutPaginationOffset)
	{
		if (PaginationOffset != -1)
		{
			OutPaginationOffset = PaginationOffset;
			return true;
		}
		return false;
	}

public:
	void SetBranch(const FString& InBranch)
	{
		Branch = InBranch;
	}

	const FString& GetBranch()
	{
		return Branch;
	}

	const TArray<FSourceControlChangelistRef>& GetSubmittedChangelists() const { return Changelists; }

	void AddSubmittedChangelist(FSourceControlChangelistRef Changelist)
	{
		Changelists.Add(Changelist);
	}

private:
	FDateTime DateFrom = FDateTime::MinValue();
	FDateTime DateTo = FDateTime::MinValue();
	bool Owned = false;
	FString Branch;

	int32 PaginationLimit = -1;
	int32 PaginationOffset = -1;

	TArray<FSourceControlChangelistRef> Changelists;
};

/**
 * This operations query the source control to extract all the details available for a given changelist. The operations returns a collection of key/value corresponding to the details available. 
 * The list of key/value is specific to the source control implementation.
 */
class FGetChangelistDetails : public FSourceControlOperationBase
{
public:
	FGetChangelistDetails() = default;

	explicit FGetChangelistDetails(FString&& InChangelistNumber)
		: ChangelistNumber(MoveTemp(InChangelistNumber))
	{

	}

	explicit FGetChangelistDetails(FStringView InChangelistNumber)
		: ChangelistNumber(InChangelistNumber)
	{

	}

	virtual ~FGetChangelistDetails() = default;

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetChangelistDetails";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetChangelistDetails", "Retrieving changelist details from Revision Control...");
	}

	const FString& GetChangelistNumber() { return ChangelistNumber; }

	const TArray<TMap<FString, FString>>& GetChangelistDetails() { return OutChangelistDetails; }
	
	void SetChangelistNumber(const FString& InChangelistNumber)
	{
		ChangelistNumber = InChangelistNumber;
	}

	void SetChangelistDetails(TArray<TMap<FString, FString>>&& InChangelistDetails)
	{
		OutChangelistDetails = MoveTemp(InChangelistDetails);
	}

private:
	FString ChangelistNumber;
	TArray<TMap<FString, FString>> OutChangelistDetails;
};

/**
 * Operation used to update the source control status of changelist(s)
 */
class FUpdatePendingChangelistsStatus : public FSourceControlOperationBase
{
public:
	void SetUpdateFilesStates(bool bInUpdateFilesStates)
	{
		bUpdateFilesStates = bInUpdateFilesStates;
	}

	bool ShouldUpdateFilesStates() const
	{
		return bUpdateFilesStates;
	}

	void SetUpdateShelvedFilesStates(bool bInUpdateShelvedFilesStates)
	{
		bUpdateShelvedFilesStates = bInUpdateShelvedFilesStates;
	}

	bool ShouldUpdateShelvedFilesStates() const
	{
		return bUpdateShelvedFilesStates;
	}

	void SetUpdateAllChangelists(bool bInUpdateAllChangelists)
	{
		bUpdateAllChangelists = bInUpdateAllChangelists;
		ChangelistsToUpdate.Empty();
	}

	bool ShouldUpdateAllChangelists() const
	{
		return bUpdateAllChangelists;
	}

	void SetChangelistsToUpdate(const TArray<FSourceControlChangelistRef>& InChangelistsToUpdate)
	{
		ChangelistsToUpdate = InChangelistsToUpdate;
		bUpdateAllChangelists = false;
	}

	void SetChangelistsToUpdate(const TArrayView<FSourceControlChangelistRef>& InChangelistsToUpdate)
	{
		ChangelistsToUpdate = InChangelistsToUpdate;
		bUpdateAllChangelists = false;
	}

	const TArray<FSourceControlChangelistRef>& GetChangelistsToUpdate() const
	{
		return ChangelistsToUpdate;
	}

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "UpdateChangelistsStatus";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_UpdateChangelistsStatus", "Updating changelist(s) status from Revision Control...");
	}

private:
	bool bUpdateFilesStates = false;
	bool bUpdateShelvedFilesStates = false;
	bool bUpdateAllChangelists = false;

	TArray<FSourceControlChangelistRef> ChangelistsToUpdate;
};

/**
* Operation used to create a new changelist
*/
class FNewChangelist : public FSourceControlOperationBase
{
public:
	virtual FName GetName() const override
	{
		return "NewChangelist";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_NewChangelist", "Creating new changelist from Revision Control...");
	}

	void SetDescription(const FText& InDescription)
	{
		Description = InDescription;
	}

	const FText& GetDescription() const
	{
		return Description;
	}

	void SetNewChangelist(FSourceControlChangelistPtr InNewChangelist)
	{
		NewChangelist = InNewChangelist;
	}

	const FSourceControlChangelistPtr& GetNewChangelist() const
	{
		return NewChangelist;
	}

protected:
	/** Description of the changelist */
	FText Description;
	FSourceControlChangelistPtr NewChangelist;
};

/**
 * Operation used to delete an empty changelist
 */
class FDeleteChangelist : public FSourceControlOperationBase
{
public:
	virtual FName GetName() const override
	{
		return "DeleteChangelist";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_DeleteChangelist", "Deleting a changelist from Revision Control...");
	}
};

/**
 * Operation to change the description of a changelist
 */
class FEditChangelist : public FSourceControlOperationBase
{
public:
	virtual FName GetName() const override
	{
		return "EditChangelist";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_EditChangelist", "Editing a changelist from Revision Control...");
	}

	void SetDescription(const FText& InDescription)
	{
		Description = InDescription;
	}

	const FText& GetDescription() const
	{
		return Description;
	}

protected:
	/** Description of the changelist */
	FText Description;
};

/**
 * Operation to revert unchanged file(s) or all unchanged files in a changelist
 */
class FRevertUnchanged : public FSourceControlOperationBase
{
public:
	virtual FName GetName() const override
	{
		return "RevertUnchanged";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_RevertUnchanged", "Reverting unchanged files from Revision Control...");
	}
};

/**
 * Operation used to move files between changelists
 */
class FMoveToChangelist : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "MoveToChangelist";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_MoveToChangelist", "Moving files to target changelist...");
	}
};

/**
 * Operation used to shelve files in a changelist
 */
class FShelve : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Shelve";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_ShelveOperation", "Shelving files in changelist...");
	}

	void SetDescription(const FText& InDescription)
	{
		Description = InDescription;
	}

	const FText& GetDescription() const
	{
		return Description;
	}

private:
	/** Description of the changelist, will be used only to create a new changelist when needed */
	FText Description;
};

/**
 * Operation used to unshelve files from a changelist
 */
class FUnshelve : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override 
	{
		return "Unshelve";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_UnshelveOperation", "Unshelving files from changelist...");
	}
};

/**
 * Operation used to delete shelved files from a changelist
 */
class FDeleteShelved : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "DeleteShelved";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_DeleteShelvedOperation", "Deleting shelved files from changelist...");
	}
};

/**
 * Operation used to download a file from the source control server directly rather
 * than sync it. This should not change the state of anything locally on the client.
 */
class FDownloadFile : public FSourceControlOperationBase
{
public:
	enum class UE_DEPRECATED(5.6, "Enum has been removed, call FSourceControlOperationBase::SetEnableErrorLogging directly instead") EVerbosity
	{
		/** No logging when the command is run. */
		None,
		/** Log the full cmdline when the command is run. */
		Full
	};

	/** 
	 * This version of FDownloadFile will download the files and keep them in memory,
	 * which can then be accessed by calling ::GetFileData.
	 */
	FDownloadFile() = default;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDownloadFile(EVerbosity InVerbosity)
	{
		if (InVerbosity == EVerbosity::None)
		{
			SetEnableErrorLogging(false);
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "DownloadFile";
	}

	// ISourceControlOperation interface
	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_PrintOperation", "Downloading file from server...");
	}

	// ISourceControlOperation interface
	virtual bool CanBeCalledFromBackgroundThreads() const override
	{
		return true;
	}
	
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Enum has been removed, use the overload without InVerbosity instead")
	SOURCECONTROL_API FDownloadFile(FStringView InTargetDirectory, EVerbosity InVerbosity);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** This version of FDownloadFile will download the files to the given target directory */
	SOURCECONTROL_API FDownloadFile(FStringView InTargetDirectory);

	/**
	 * Returns the directory that the files will (or have) been downloaded to. This path
	 * will be empty if no target directory was given, in which case the files can be
	 * accessed via the ::GetFileData method.
	 */
	FString GetTargetDirectory() const
	{
		return TargetDirectory;
	}

	/**
	 * If no target directory was given to download the files too then the command will keep
	 * the files in memory which can be accessed via this method. If the file failed to download
	 * then a null FSharedBuffer will be returned.
	 */
	SOURCECONTROL_API FSharedBuffer GetFileData(const FStringView& Filename);

	/** Return true if the command should log its operations, otherwise false */
	bool ShouldLogToStdOutput() const
	{
		return !EnumHasAllFlags(Flags, EFlags::DisableErrorLogging);
	}

	/** 
	 * Do not call outside of source control implementations. Used to add the file in memory
	 * once downloaded so that the caller can get access to it.
	 */
	void __Internal_AddFileData(const FString& Filename, FSharedBuffer FileData)
	{
		FileDataMap.Add(Filename, FileData);
	}

	UE_DEPRECATED(5.3, "Replaced by __Internal_AddFileData but you shouldn't be calling this anyway")
	void AddFileData(const FString& Filename, FSharedBuffer FileData)
	{
		__Internal_AddFileData(Filename, FileData);
	}

private:

	FString TargetDirectory;
	TMap<FString, FSharedBuffer> FileDataMap;
};

/**
 * Operation used to get info about file(s) at a revision. CL number, datetime, and author will be returned.
 */
class FGetSourceControlRevisionInfo : public FSourceControlOperationBase
{
public:

	FGetSourceControlRevisionInfo() = default;

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetSourceControlRevisionInfo";
	}

	// ISourceControlOperation interface
	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetRevisionInfoPending", "Getting SourceControl info for file revision...");
	}

	// ISourceControlOperation interface
	virtual bool CanBeCalledFromBackgroundThreads() const override
	{
		return true;
	}

	/** Return true if the command should log its operations, otherwise false */
	bool ShouldLogToStdOutput() const
	{
		return !EnumHasAllFlags(Flags, EFlags::DisableErrorLogging);
	}

	FString GetChangelist() const
	{
		return Changelist;
	}

	FDateTime GetDateTime() const
	{
		return DateTime;
	}

	FString GetAuthor() const
	{
		return Author;
	}

	/** 
	 * Do not call outside of source control implementations. Used to add the results
	 * so that the caller can get access to it.
	 */
	void InternalSetResults(const FString& InChangelist, const FDateTime& InDateTime, const FString& InAuthor)
	{
		Changelist = InChangelist;
		DateTime = InDateTime;
		Author = InAuthor;
	}

private:
	FString Changelist;
	FDateTime DateTime;
	FString Author;
};

/** 
 * Operation used to create a new workspace if the source control system supports this functionality.
 */
class FCreateWorkspace : public FSourceControlOperationBase
{
public:
	enum class EType
	{
		/* Default */
		Writeable = 0,
		/* Has a short life span and can only query/read/sync files */
		ReadOnly,
		/* Has a short life span but is allowed full functionality */
		Partitioned
	};

	using FClientViewMapping = TPair<FString,FString>;

	FCreateWorkspace() = delete;

	/**
	 * Constructor
	 * 
	 * @param WorkspaceName	The name of the workspace to create
	 * @param WorkspaceRoom	The file path to the workspace root (can be relative to the project)
	 */
	SOURCECONTROL_API FCreateWorkspace(FStringView WorkspaceName, FStringView WorkspaceRoot);
	
	/** Set the description to be used by the workspace, if left unset then the default description will be used */
	void SetDescription(FStringView Desciption)
	{
		WorkspaceDescription = Desciption;
	}

	/** Set the stream to be used by the workspace, if left unset then a classic depot with ClientView will be used */
	void SetStream(FStringView Stream)
	{
		WorkspaceStream = Stream;
	}

	/**
	 * Add a new mapping for the client spec in the native format of the current source control provider.
	 * These will be written to the client spec in the order that they are added.
	 * 
	 * @param DepotPath		The path in the source control depot to map from
	 * @param ClientPath	The path on the local client to map too
	 */
	void AddNativeClientViewMapping(FStringView DepotPath, FStringView ClientPath)
	{
		ClientView.Emplace(DepotPath, ClientPath);
	}

	/** Remove all currently set client-view mappings */
	void ClearClientViewMappings()
	{
		ClientView.Empty();
	}

	/** Set the type of workspace to be created. @see FCreateWorkspace::EType */
	void SetType(EType InType)
	{
		Type = InType;
	}

	const FString& GetWorkspaceName() const
	{
		return WorkspaceName;
	}

	const FString& GetWorkspaceRoot() const
	{
		return WorkspaceRoot;
	}

	const FString& GetWorkspaceDescription() const
	{
		return WorkspaceDescription;
	}

	const FString& GetWorkspaceStream() const
	{
		return WorkspaceStream;
	}

	const TArray<FClientViewMapping> GetClientView() const
	{
		return ClientView;
	}

	EType GetType() const
	{
		return Type;
	}

	// ISourceControlOperation interface

	virtual FName GetName() const override
	{
		return "CreateWorkspace";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_CreateWorkspaceOperation", "Creating a workspace...");
	}

protected:

	FString WorkspaceName;
	FString WorkspaceRoot;

	FString WorkspaceDescription;
	FString WorkspaceStream;

	EType Type = EType::Writeable;

	TArray<FClientViewMapping> ClientView;
};

/** Operation used to delete a workspace */
class FDeleteWorkspace : public FSourceControlOperationBase
{
public:
	FDeleteWorkspace() = delete;

	/** 
	 * Constructor
	 * 
	 * @param InWorkspaceName The name of the workspace to be deleted
	 */
	FDeleteWorkspace(FStringView InWorkspaceName)
		: WorkspaceName(InWorkspaceName)
	{

	}

	const FString& GetWorkspaceName() const
	{
		return WorkspaceName;
	}

	// ISourceControlOperation interface

	virtual FName GetName() const override
	{
		return "DeleteWorkspace";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_DeleteWorkspaceOperation", "Deleting a workspace...");
	}

protected:

	FString WorkspaceName;
};

/** Information on a source control workspace */
struct FScWorkspaceInfo
{
	FString Name;
	FString RootDir;
};

/**
 * Operation used to get all workspaces for the current host if the source control system supports this functionality.
 */
class FGetWorkspaces : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetWorkspaces";
	}

public:
	/** Output array of workspace info */
	TArray<FScWorkspaceInfo> Results;
};

/**
 * This operation uses p4v print command to get file from specified depot by shelved changelist or file revision returns a package filename
 */
class FGetFile : public FSourceControlOperationBase
{
public:

	FGetFile(const FString& InChangelistNumber, const FString& InRevisionNumber, const FString& InDepotFilePath, bool bInIsShelve = false)
		: ChangelistNumber(InChangelistNumber)
		, RevisionNumber(InRevisionNumber)
		, DepotFilePath(InDepotFilePath)
		, bIsShelve(bInIsShelve)
	{}

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "GetFile";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_GetFile", "Retrieving file from Revision Control...");
	}

	const FString& GetChangelistNumber() const { return ChangelistNumber; }

	const FString& GetRevisionNumber() const { return RevisionNumber; }

	const FString& GetDepotFilePath() const { return DepotFilePath; }

	bool IsShelve() const { return bIsShelve; }

	const FString& GetOutPackageFilename() const { return OutPackageFilename; }

	void SetOutPackageFilename(const FString& InOutPackageFilename)
	{
		OutPackageFilename = InOutPackageFilename;
	}

private:

	FString ChangelistNumber;
	FString RevisionNumber;
	FString DepotFilePath;
	bool bIsShelve;

	FString OutPackageFilename;
};

/**
* This operation will get info about the location of a file within a source control provider. The input can be a local or remote path
* and the result contain both local and remote paths
*/
class FWhere : public FSourceControlOperationBase
{
public:
	struct FileInfo
	{
		FString LocalPath;
		FString RemotePath;
	};

	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Where";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Where", "Locating file(s) in Revision Control...");
	}

	void SetFiles(TArray<FileInfo>&& InFiles) { Files = MoveTemp(InFiles); }
	const TArray<FileInfo>& GetFiles() const { return Files; }

private:
	TArray<FileInfo> Files;
};

/**
 * This operation will save files into revision control without actually doing a check-in
 */
class FSave : public FSourceControlOperationBase
{
public:
	// ISourceControlOperation interface
	virtual FName GetName() const override
	{
		return "Save";
	}

	virtual FText GetInProgressString() const override
	{
		return LOCTEXT("SourceControl_Save", "Saving file(s) into Revision Control...");
	}
};

#undef LOCTEXT_NAMESPACE
