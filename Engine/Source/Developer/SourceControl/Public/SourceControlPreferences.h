// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "SourceControlPreferences.generated.h"

#define UE_API SOURCECONTROL_API

class FName;
class UObject;


/** Settings for the Source Control Integration */
UCLASS(MinimalAPI, config = Editor, defaultconfig, meta = (DisplayName = "Revision Control", Keywords = "Source Control"))
class USourceControlPreferences : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/** Helper to access easily Enable Validation Tag setting */
	static UE_API bool IsValidationTagEnabled();

	/** Helper to access easily Should Delete New Files On Revert setting */
	static UE_API bool ShouldDeleteNewFilesOnRevert();

	/** Helper to access easily Enable Uncontrolled Changelists setting */
	static UE_API bool AreUncontrolledChangelistsEnabled();

	/** Helper to access easily Requires Revision Control To Rename Localizable Assets setting */
	static UE_API bool RequiresRevisionControlToRenameLocalizableAssets();

public:
	/** If enabled, adds a tag in changelist descriptions when they are validated */
	UPROPERTY(config, EditAnywhere, Category = SourceControl, meta = (ToolTip = "Adds validation tag to changelist description on submit."))
	bool bEnableValidationTag = true;

	/** If enabled, deletes new files when reverted. */
	UPROPERTY(config, EditAnywhere, Category = SourceControl, meta = (ToolTip = "Deletes new files when reverted."))
	bool bShouldDeleteNewFilesOnRevert = true;

	/** Enables Uncontrolled Changelists features. */
	UPROPERTY(config, EditAnywhere, Category = SourceControl, meta = (ToolTip = "Enables Uncontrolled Changelists features. The editor must be restarted for the change to be fully taken into account.", ConfigRestartRequired = true))
	bool bEnableUncontrolledChangelists = true;

	/** List of lines to add to any collection on checkin */
	UPROPERTY(config, EditAnywhere, Category = SourceControl)
	TArray<FString> CollectionChangelistTags;

	/** Map of collection names and additional text to apply to changelist descriptions when checking them in */
	UPROPERTY(config, EditAnywhere, Category = SourceControl, meta=(MultiLine=true))
	TMap<FName, FString> SpecificCollectionChangelistTags;

	/** Revision Control is a requirement to move/rename/delete localizable assets */
	UPROPERTY(config, EditAnywhere, DisplayName = "Requires Revision Control To Manage Localizable Assets", Category = Internationalization, meta = (ToolTip = "If the project uses Localization and its localized content might not be on disk (but still in your Revision Control), then enabling this feature will require a Revision Control Provider to be configured to move/rename/delete files to be sure no localized variants become orphaned. (Works only with Perforce for now)"))
	bool bRequiresRevisionControlToRenameLocalizableAssets = false;
};

#undef UE_API
