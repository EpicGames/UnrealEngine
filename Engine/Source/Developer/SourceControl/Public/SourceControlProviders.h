// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

namespace SourceControlProviders
{
	/** Returns the default SourceControl provider name, which is the one indicating 'disabled'. */
	static const TCHAR* GetDefaultProviderName()
	{
		return TEXT("None");
	}

	/** Returns the URC SourceControl provider name */
	static const TCHAR* GetUrcProviderName()
	{
		return TEXT("Unreal Revision Control");
	}	
	
	/** Returns the old Skein SourceControl provider name */
    static const TCHAR* GetLegacySkeinProviderName()
	{
		return TEXT("Skein");
	}

	/** Returns the Perforce SourceControl provider name */
	static const TCHAR* GetPerforceProviderName()
	{
		return TEXT("Perforce");
	}
}