// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/EnumClassFlags.h"

/** Accumulated error and info messages for a source control operation.  */
struct FSourceControlResultInfo
{
	enum class EAdditionalErrorContext : uint32
	{
		None				= 0,
		/** A connection to the server could not be established */
		ConnectionFailed	= 1 << 0,
		/** The connection to the server dropped before the operation could be completed */
		ConnectionDropped	= 1 << 1
	};

	FRIEND_ENUM_CLASS_FLAGS(EAdditionalErrorContext);

	/** Append any messages from another FSourceControlResultInfo, ensuring to keep any already accumulated info. */
	void Append(const FSourceControlResultInfo& InResultInfo)
	{
		InfoMessages.Append(InResultInfo.InfoMessages);
		ErrorMessages.Append(InResultInfo.ErrorMessages);
		Tags.Append(InResultInfo.Tags);

		AdditionalErrorContext |= InResultInfo.AdditionalErrorContext;
	}

	SOURCECONTROL_API void OnConnectionFailed();
	SOURCECONTROL_API void OnConnectionDroped();

	SOURCECONTROL_API bool DidConnectionFail() const;

	bool HasErrors() const
	{
		return !ErrorMessages.IsEmpty();
	}

	/** Info and/or warning message storage */
	TArray<FText> InfoMessages;

	/** Potential error message storage */
	TArray<FText> ErrorMessages;

	/** Additional arbitrary information attached to the command */
	TArray<FString> Tags;

private:
	/** Contains additional info about any errors encountered */
	EAdditionalErrorContext AdditionalErrorContext = EAdditionalErrorContext::None;
};

ENUM_CLASS_FLAGS(FSourceControlResultInfo::EAdditionalErrorContext);
