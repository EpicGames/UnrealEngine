// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISourceControlProvider.h"

namespace SourceControlAutomationCommon
{
	/**
	* Helper class for receiving the results of async source control operations
	*/
	class FAsyncCommandHelper
	{
	public:
		FAsyncCommandHelper(const FString& InParameter = FString())
			: Parameter(InParameter)
			, bDispatched(false)
			, bDone(false)
			, bSuccessful(false)
		{
		}

		void SourceControlOperationComplete(const FSourceControlOperationRef& Operation, ECommandResult::Type InResult)
		{
			bDone = true;
			bSuccessful = InResult == ECommandResult::Succeeded;
		}

		const FString& GetParameter() const
		{
			return Parameter;
		}

		bool IsDispatched() const
		{
			return bDispatched;
		}

		void SetDispatched()
		{
			bDispatched = true;
		}

		bool IsDone() const
		{
			return bDone;
		}

		bool IsSuccessful() const
		{
			return bSuccessful;
		}

	private:
		/** Parameter we perform this operation with, if any */
		FString Parameter;

		/** Whether the async operation been issued */
		bool bDispatched;

		/** Whether the async operation has completed */
		bool bDone;

		/** Whether the operation was successful */
		bool bSuccessful;
	};
}
