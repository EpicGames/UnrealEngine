// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class SourceControl : ModuleRules
{
	public SourceControl(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"DeveloperSettings",
			}
		);

		if (Target.bUsesSlate)
		{
			PublicDefinitions.Add("SOURCE_CONTROL_WITH_SLATE=1");
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"SlateCore",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"Slate",
					"RenderCore",
					"RHI"
				}
			);
		}
		else
		{
			PublicDefinitions.Add("SOURCE_CONTROL_WITH_SLATE=0");
		}

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry"
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"EditorFramework",
					"Engine",
					"UnrealEd",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"AssetTools"
				}
			);

			CircularlyReferencedDependentModules.Add("UnrealEd");
		}

		if (Target.bBuildDeveloperTools)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MessageLog",
				}
			);
		}

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrecompileForTargets = PrecompileTargetsType.Any;
		}

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
	}
}
